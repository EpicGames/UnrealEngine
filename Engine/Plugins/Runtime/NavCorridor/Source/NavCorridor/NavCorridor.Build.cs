// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NavCorridor : ModuleRules
	{
		public NavCorridor(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
			new string[] {
				"AIModule",
				"Core",
				"CoreUObject",
				"Engine",
				"NavigationSystem",
			}
			);
		}
	}
}
