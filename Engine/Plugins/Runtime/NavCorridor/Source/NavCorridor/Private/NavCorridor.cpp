// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavCorridor.h"

#include "AI/Navigation/NavAgentInterface.h"
#include "GeomUtils.h"
#include "NavigationData.h"
#include "NavMesh/NavMeshPath.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavCorridor)

#if ENABLE_VISUAL_LOG
DEFINE_LOG_CATEGORY_STATIC(LogNavCorridorQuads, Log, All);
DEFINE_LOG_CATEGORY_STATIC(LogNavCorridorEdges, Log, All);
DEFINE_LOG_CATEGORY_STATIC(LogNavCorridorPath, Log, All);
#endif // ENABLE_VISUAL_LOG

//-------------------------------------------------------
// Private helper functions for the nav corridor.
//-------------------------------------------------------

namespace UE::NavCorridor::Private
{
#if ENABLE_VISUAL_LOG
	static bool bEnableNavCorridorVLog = false;
	static FAutoConsoleVariableRef CVarNavCorridorDebugEnableVisualLogging(TEXT("ai.nav.EnableNavCorridorVLog"), bEnableNavCorridorVLog, TEXT("If set, NavCorridor debug information will be logged in the Visual Logger."), ECVF_Cheat);
#endif // ENABLE_VISUAL_LOG

	static bool bUseSimplifyConcavePortalsFix = true;
	static FAutoConsoleVariableRef CVarUseSimplifyConcavePortalsFix(TEXT("ai.nav.bUseSimplifyConcavePortalsFix"),
		bUseSimplifyConcavePortalsFix, TEXT("True by default. Set to false to revert to previous corridor behavior when simplifying."), ECVF_Cheat);
	
	/** Epsilon used when testing if 3 points for a convex corner.
	 * The check is done using signed distance of the corner points to the segment from first to last points.
	 * This is how much it can be concave. */
	constexpr FVector::FReal ConvexEpsilon = 0.01;

	/** A segment in bilinear patch UV space. */
	struct FUVSegment
	{
		FUVSegment() = default;
		FUVSegment(const FVector2D InStartUV, const FVector2D InEndUV) : StartUV(InStartUV), EndUV(InEndUV) {}
			
		FVector2D StartUV;
		FVector2D EndUV;
	};

	/** @return approximate distance between two segments. */
	static FVector::FReal ApproxDistanceSegmentSegment(const FVector StartA, const FVector EndA, const FVector StartB, const FVector EndB)
	{
		const FVector2D Seg(EndA - StartA);
		const FVector Mid = (StartB + EndB) * 0.5;
		if (Seg.IsNearlyZero())
		{
			return FVector::DistSquared2D(Mid, StartA);
		}
		return FMath::Abs(UE::AI::SignedDistancePointLine2D(Mid, StartA, EndA));
	}

	/** Adds unique value to array with epsilon test. */
	static void AddUnique(TArray<FVector::FReal>& Values, const FVector::FReal NewValue)
	{
		using FReal = FVector::FReal;
		constexpr FReal Epsilon = 0.0001;

		for (const FReal Value : Values)
		{
			if (FMath::IsNearlyEqual(Value, NewValue, Epsilon))
			{
				return;
			}
		}

		Values.Add(NewValue);
	}

	/**
	 * Calculate segment forward and left directions of a give path.
	 * Note: Number of segments is one less than number of points. 
	 * @param Points path points
	 * @param OutSegmentDirs segment forward directions
	 * @param OutSegmentLefts segment left directions
	 */
	static void CalculateSegmentDirections(const TConstArrayView<FNavPathPoint> Points, TArray<FVector>& OutSegmentDirs, TArray<FVector>& OutSegmentLefts)
	{
		for (int32 PointIndex = 0; PointIndex < Points.Num() - 1; PointIndex++)
		{
			const FNavPathPoint& CurrPt = Points[PointIndex];
			const FNavPathPoint& NextPt = Points[PointIndex + 1];
			const FVector Dir = NextPt.Location - CurrPt.Location;
			const FVector Left = FVector::CrossProduct(Dir, FVector::UpVector);
			OutSegmentDirs.Add(Dir.GetSafeNormal2D());
			OutSegmentLefts.Add(Left.GetSafeNormal2D());
		}
	}

	/**
	 * Calculates thick corridor in 2D based on path points. The corridor is expanded symmetrically around the path.
	 * Each sector (area between two portals) is guaranteed to be convex and non-intersecting (up to floating point precision),
	 * but otherwise the resulting path may overlap itself.
	 * @param Points path points
	 * @param SegmentLefts left direction of each segment on the path (see CalculateSegmentDirections())
	 * @param Width width of the corridor to generate.
	 * @param OutPortals resulting corridor portals.
	 */
	static void CalculateCorridorPortals(const TConstArrayView<FNavPathPoint> Points, const TConstArrayView<int32> PinnedPointsIndices, const TArrayView<FVector> SegmentLefts, const FVector::FReal Width, TArray<FNavCorridorPortal>& OutPortals)
	{
		using FReal = FVector::FReal;
		TArray<FVector> PointMiterDirs;

		PointMiterDirs.Reserve(Points.Num());
		OutPortals.Reserve(Points.Num());
		
		const FReal HalfWidth = Width * 0.5;

		// Calculate miter direction for each point.
		PointMiterDirs.Add(SegmentLefts[0]);

		for (int32 SegmentIndex = 0; SegmentIndex < SegmentLefts.Num() - 1; SegmentIndex++)
		{
			const FVector CurrLeft = SegmentLefts[SegmentIndex];
			const FVector NextLeft = SegmentLefts[SegmentIndex + 1];

			FVector MiterDir = 0.5 * (CurrLeft + NextLeft);
			const FReal MidSquared = FVector::DotProduct(MiterDir, MiterDir);
			if (MidSquared > UE_KINDA_SMALL_NUMBER)
			{
				// When the segments are opposite direction, scale approaches infinity. Clamp to some arbitrary big value to avoid crazy geometry.
				constexpr FReal MaxScale = 20.0; 
				const FReal Scale = FMath::Min(1.0 / MidSquared, MaxScale);
				MiterDir *= Scale;
			}

			PointMiterDirs.Add(MiterDir);
		}

		PointMiterDirs.Add(SegmentLefts.Last());

		check(PointMiterDirs.Num() == Points.Num());

		// Intersect segments for each sector, and keep the most clamped result.
		TArray<FReal> PortalLeftU;
		TArray<FReal> PortalRightU;
		PortalLeftU.Init(0.0, Points.Num());
		PortalRightU.Init(1.0, Points.Num());

		for (int32 PointIndex = 0; PointIndex < Points.Num() - 1; PointIndex++)
		{
			const FVector CurrLoc = Points[PointIndex].Location;
			const FVector CurrMiterLeft = PointMiterDirs[PointIndex];
			const FVector NextLoc = Points[PointIndex + 1].Location;
			const FVector NextMiterLeft = PointMiterDirs[PointIndex + 1];
			
			const FVector CurrBase = CurrLoc + CurrMiterLeft * HalfWidth;
			const FVector CurrDelta = CurrMiterLeft * -Width;
			const FVector NextBase = NextLoc + NextMiterLeft * HalfWidth;
			const FVector NextDelta = NextMiterLeft * -Width;

			// Check if the portals intersect. The logic below ensures that the resulting sector is convex.
			// This means that we need to adjust the portals, when the infinite lines of the portals intersect
			// at either portal segment body. The segment middle is used as an indicator to check which end needs to be pruned. 
			//
			//   Curr       /            Curr        Intersection
			//   -------   /             -------  x
			//            /                      /
			//           / Next                 / Next
			//          /                      /
			//
			FReal CurrT = 0.0, NextT = 0.0;
			const bool bIntersects = UE::AI::IntersectLineLine2D(CurrBase, CurrBase + CurrDelta, NextBase, NextBase + NextDelta, CurrT, NextT); 
			if (bIntersects && ((CurrT > 0.0 && CurrT < 1.0) || (NextT > 0.0 && NextT < 1.0)))
			{
				if (CurrT < 0.5)
				{
					PortalLeftU[PointIndex] = FMath::Max(PortalLeftU[PointIndex], CurrT);
					PortalLeftU[PointIndex + 1] = FMath::Max(PortalLeftU[PointIndex + 1], NextT);
				}
				else
				{
					PortalRightU[PointIndex] = FMath::Min(PortalRightU[PointIndex], CurrT);
					PortalRightU[PointIndex + 1] = FMath::Min(PortalRightU[PointIndex + 1], NextT);
				}
			}
		}

		// Output the portals.
		for (int32 PointIndex = 0; PointIndex < Points.Num(); PointIndex++)
		{
			const FVector Loc = Points[PointIndex].Location;
			const FVector MiterLeft = PointMiterDirs[PointIndex];
			const FVector Base = Loc + MiterLeft * HalfWidth;
			const FVector Delta = MiterLeft * -Width;
			if (PortalLeftU[PointIndex] > PortalRightU[PointIndex])
			{
				PortalLeftU[PointIndex] = PortalRightU[PointIndex] = (PortalLeftU[PointIndex] + PortalRightU[PointIndex]) * 0.5;
			}
			
			FNavCorridorPortal& Portal = OutPortals.AddDefaulted_GetRef();
			Portal.Left = Base + Delta * PortalLeftU[PointIndex];
			Portal.Right = Base + Delta * PortalRightU[PointIndex];
			Portal.Location = Loc;
			Portal.bIsPinnedLocation = PinnedPointsIndices.Contains(PointIndex);
		}
	}

	/**
	 * Clips boundary edges into given sector (expanded segment from the original path) in 2D, arranges them to left and right obstacles, and projects them into the sectors UV space.
	 * @param SegStart segment start position
	 * @param SegForward segment forward direction
	 * @param SegLeft segment left direction
	 * @param Quad convex sector boundary
	 * @param Edges edges to clip (2 vectors per edge).
	 * @param OutLeftSegments edges in UV space left of the segment 
	 * @param OutRightSegments edges in UV space right of the segment
	 */
	static void ClipEdgesToSector(const UObject* PathQuerier, const FVector SegStart, const FVector SegForward, const FVector SegLeft, TConstArrayView<FVector> Quad, TConstArrayView<FVector> Edges,
						  TArray<FUVSegment>& OutLeftSegments, TArray<FUVSegment>& OutRightSegments)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NavCorridor_ClipEdgesToSector);

		using FReal = FVector2D::FReal;
		
		for (int32 EdgeIndex = 0; EdgeIndex < Edges.Num(); EdgeIndex += 2)
		{
			const FVector EdgeStart = Edges[EdgeIndex];
			const FVector EdgeEnd = Edges[EdgeIndex + 1];

			// First check if the obstacle edge intersects the sector quad.
			FReal TMin = 0.0f, TMax = 1.0f;
			int32 SegMin = INDEX_NONE, SegMax = INDEX_NONE;

			if (UE::AI::IntersectSegmentPoly2D(EdgeStart, EdgeEnd, Quad, TMin, TMax, SegMin, SegMax))
			{
				const FVector StartPos = FMath::Lerp(EdgeStart, EdgeEnd, TMin);
				const FVector EndPos = FMath::Lerp(EdgeStart, EdgeEnd, TMax);

				// Skip degenerate segments (can happen when edge is close to a quad vertex)
				if (FVector::Dist(StartPos, EndPos) < UE_KINDA_SMALL_NUMBER)
				{
					continue;
				}

				// Categorize the clipped segment to left or right to the path segment.
				const FReal StartDist = FVector::DotProduct(SegLeft, StartPos - SegStart);
				const FReal EndDist = FVector::DotProduct(SegLeft,  EndPos - SegStart);

				const FReal DistOnLeft = FMath::Max(StartDist, EndDist);	// Positive distance to left of the segment 
				const FReal DistOnRight = FMath::Max(-StartDist, -EndDist);	// Positive distance to right of the segment

				static constexpr FReal ParallelEps = 1.0;

				bool bIsLeft = false;
				
				if (DistOnLeft < ParallelEps && DistOnRight < ParallelEps)
				{
					// Edge that is parallel to the path segment, segment direction (winding) dictates which side it should go to.
					const FVector EdgeDir = EndPos - StartPos;
					bIsLeft = FVector::DotProduct(SegForward, EdgeDir) < 0.0;
				}
				else
				{
					// Choose the side based on which side the segment is more into. 
					bIsLeft = DistOnLeft > DistOnRight;
				}

				
				// Convert the world position into the quads UV space.
				const FVector2D StartUV = UE::AI::InvBilinear2DClamped(StartPos, Quad[0], Quad[1], Quad[2], Quad[3]);
				const FVector2D EndUV = UE::AI::InvBilinear2DClamped(EndPos, Quad[0], Quad[1], Quad[2], Quad[3]);

				bool bAdded = false;
				
				// Back face culling (assume the edges are oriented), and add.
				const FReal DeltaV = EndUV.Y - StartUV.Y;
				if (bIsLeft)
				{
					if (DeltaV < -UE_KINDA_SMALL_NUMBER)
					{
						OutLeftSegments.Emplace(EndUV, StartUV); // Vertices in UV segments are assumed to be in ascending order 
						bAdded = true;
					}
				}
				else
				{
					if (DeltaV > UE_KINDA_SMALL_NUMBER)
					{
						OutRightSegments.Emplace(StartUV, EndUV);
						bAdded = true;
					}
				}

#if ENABLE_VISUAL_LOG
				if (UE::NavCorridor::Private::bEnableNavCorridorVLog)
				{
					const FVector Dir = EndPos - StartPos;
					const FVector Left = FVector::CrossProduct(Dir, FVector::UpVector).GetSafeNormal();
					const FVector Mid = (StartPos + EndPos) * 0.5f;

					if (bIsLeft)
					{
						const FVector Offset(0,0, 15);
						const FVector Offset2(0,0, 17);
						UE_VLOG_SEGMENT_THICK(PathQuerier, LogNavCorridorEdges, Log, StartPos + Offset, EndPos + Offset2, FColor::Blue, 2, TEXT_EMPTY);
						UE_VLOG_SEGMENT_THICK(PathQuerier, LogNavCorridorEdges, Log, Mid + Offset, Mid + Left * 10.0f + Offset, bAdded ? FColor::Blue : FColor::Red, 1, TEXT_EMPTY);
					}
					else
					{
						const FVector Offset(0,0, 15);
						const FVector Offset2(0,0, 17);
						UE_VLOG_SEGMENT_THICK(PathQuerier, LogNavCorridorEdges, Log, StartPos + Offset, EndPos + Offset2, FColor::Green, 2, TEXT_EMPTY);
						UE_VLOG_SEGMENT_THICK(PathQuerier, LogNavCorridorEdges, Log, Mid + Offset, Mid + Left * 10.0f + Offset, bAdded ? FColor::Green : FColor::Red, 1, TEXT_EMPTY);
					}
				}
#endif // ENABLE_VISUAL_LOG
			}
		}

		Algo::Sort(OutLeftSegments, [](const FUVSegment& A, const FUVSegment& B) { return A.StartUV.Y < B.StartUV.Y; });
		Algo::Sort(OutRightSegments, [](const FUVSegment& A, const FUVSegment& B) { return A.StartUV.Y < B.StartUV.Y; });
	}

	/**
	 * Ensures that segments slopes are within specified range. Fixes internal edge slopes, and adds taper segments at the end extrema of objects.
	 * A taper segment is an angled segment that makes the obstacle edges tapered. These tapers helps to remove small sectors from the
	 * final corridor (steep angle results in a narrow sector that cannot be removed), and it helps to prevent local traps which
	 * can cause problems with avoidance, etc.
	 *
	 *         o----o
	 *        /:     \
	 *     o---o      \
	 *    /            \
	 * ../..............\.
	 *
	 * @param Segments obstacle segments
	 * @param EdgeU U value of the edge we're currently processing (left = 0.0, right = 1.0)
	 * @param SlopeOffsetV How much V is offset based on change in U. 
	 */
	static void AddTaperSegments(TArray<FUVSegment>& Segments, const FVector::FReal EdgeU, const FVector::FReal SlopeOffsetV)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NavCorridor_AddTaperSegments);

		using FReal = FVector::FReal;

		TArray<FUVSegment> SlopeSegments;
		SlopeSegments.Reserve(Segments.Num());
		
		const FReal MinSlope = 1.0 / SlopeOffsetV;

		// Find end points of edge chains. The edges are already sorted based on Y (V) coordinate.  
		TArray<bool> StartConnected;
		TArray<bool> EndConnected;
		StartConnected.Init(false, Segments.Num());
		EndConnected.Init(false, Segments.Num());

		for (int32 SegmentIndex = 0; SegmentIndex < Segments.Num() - 1; SegmentIndex++)
		{
			const FUVSegment& CurrSegment = Segments[SegmentIndex];

			for (int32 NextSegmentIndex = SegmentIndex + 1; NextSegmentIndex < Segments.Num(); NextSegmentIndex++)
			{
				const FUVSegment& NextSegment = Segments[NextSegmentIndex];
				if (CurrSegment.EndUV.Equals(NextSegment.StartUV))
				{
					EndConnected[SegmentIndex] = true;
					StartConnected[NextSegmentIndex] = true;
				}

				if (CurrSegment.EndUV.Y < NextSegment.StartUV.Y)
				{
					break;
				}
			}
		}

		// Correct segment slopes
		for (int32 SegmentIndex = 0; SegmentIndex < Segments.Num(); SegmentIndex++)
		{
			FUVSegment& CurrSegment = Segments[SegmentIndex];

			const FVector2D SegDelta = CurrSegment.EndUV - CurrSegment.StartUV;
			const FReal Slope = SegDelta.Y > UE_KINDA_SMALL_NUMBER ? (SegDelta.X / SegDelta.Y) : MAX_dbl;

			FReal StartDistance = FMath::Abs(EdgeU - CurrSegment.StartUV.X);
			FReal EndDistance = FMath::Abs(EdgeU - CurrSegment.EndUV.X);

			// If the edge is too steep edge, correct the draft.
			if (FMath::Abs(Slope) >= MinSlope)
			{
				if (StartDistance > EndDistance)
				{
					// Start is higher, keep start.
					CurrSegment.EndUV = FVector2D(EdgeU, CurrSegment.StartUV.Y + StartDistance * SlopeOffsetV);
				}
				else
				{
					CurrSegment.StartUV = FVector2D(EdgeU, CurrSegment.EndUV.Y - EndDistance * SlopeOffsetV);
				}
				StartDistance = FMath::Abs(EdgeU - CurrSegment.StartUV.X);
				EndDistance = FMath::Abs(EdgeU - CurrSegment.EndUV.X);
			}
			
			// Add a draft segment if the edge is start or end of and edge chain and not connected to edge.
			if (!StartConnected[SegmentIndex] && StartDistance > UE_KINDA_SMALL_NUMBER)
			{
				SlopeSegments.Emplace(FVector2D(EdgeU, CurrSegment.StartUV.Y - StartDistance * SlopeOffsetV), CurrSegment.StartUV);
			}
			if (!EndConnected[SegmentIndex] && EndDistance > UE_KINDA_SMALL_NUMBER)
			{
				SlopeSegments.Emplace(CurrSegment.EndUV, FVector2D(EdgeU, CurrSegment.EndUV.Y + EndDistance * SlopeOffsetV));
			}
		}

		Segments.Append(SlopeSegments);
		
		Algo::Sort(Segments, [](const FUVSegment& A, const FUVSegment& B) { return A.StartUV.Y < B.StartUV.Y; });
	}

	/** Simple iterator use to speed up consecutive samplings. */
	struct FSampleSegmentsIterator
	{
		int32 LowSegmentIndex = 0;
	};

	/**
	 * Samples height (U) of segments at the location of SampleAtV. If multiple segments are at location SampleAtV, the value that is furthest from EdgeU is returned.
	 * The iterator is used to limit the searched segments when consecutive increasing samples are taken.
	 * @param Iter sampling progress
	 * @param SampleAtV sampling location
	 * @param EdgeU U value of the edge we're currently processing (left = 0.0, right = 1.0)
	 * @param Segments segments to sample
	 * @param Quad bilinear patch used for mapping between world and UV space
	 * @param OutSegmentIndex Hit segment index, or INDEX_NONE if no hit.
	 * @return height (U value) at location SampleAtV
	 */
	static FVector::FReal SampleSegmentsHeight(FSampleSegmentsIterator& Iter, const FVector::FReal SampleAtV, const FVector::FReal EdgeU, const TConstArrayView<FUVSegment> Segments, const TConstArrayView<FVector> Quad, int32& OutSegmentIndex)
	{
		using FReal = FVector::FReal;

		OutSegmentIndex = INDEX_NONE;
		FReal BestU = EdgeU;
		FReal BestDist = 0.0;

		for (int32 SegmentIndex = Iter.LowSegmentIndex ; SegmentIndex < Segments.Num(); SegmentIndex++)
		{
			const FUVSegment& Segment = Segments[SegmentIndex];

			if (SampleAtV < (Segment.StartUV.Y - UE_KINDA_SMALL_NUMBER))
			{
				return BestU;
			}

			// Update low index, no need to check segments below this index later.
			while ((Iter.LowSegmentIndex <= SegmentIndex) && (SampleAtV > (Segments[Iter.LowSegmentIndex].EndUV.Y + UE_KINDA_SMALL_NUMBER)))
			{
				Iter.LowSegmentIndex++;
			}

			if (SampleAtV > (Segment.EndUV.Y + UE_KINDA_SMALL_NUMBER))
			{
				continue;
			}

			FReal U = EdgeU;
			if (SampleAtV < (Segment.StartUV.Y + UE_KINDA_SMALL_NUMBER))
			{
				// Hit Start point
				U = Segment.StartUV.X;
			}
			else if (SampleAtV > (Segment.EndUV.Y - UE_KINDA_SMALL_NUMBER))
			{
				// Hit end point
				U = Segment.EndUV.X;
			}
			else
			{
				// In between
				const FReal DeltaV = Segment.EndUV.Y - Segment.StartUV.Y;
				if (DeltaV < UE_KINDA_SMALL_NUMBER)
				{
					U = Segment.StartUV.X;
				}
				else
				{
					const FReal T = (SampleAtV - Segment.StartUV.Y) / DeltaV;
					const FVector StartPos = UE::AI::Bilinear(Segment.StartUV, Quad[0], Quad[1], Quad[2], Quad[3]);
					const FVector EndPos = UE::AI::Bilinear(Segment.EndUV, Quad[0], Quad[1], Quad[2], Quad[3]);
					const FVector HitPos = FMath::Lerp(StartPos, EndPos, T);
					const FVector2D HitUV = UE::AI::InvBilinear2DClamped(HitPos, Quad[0], Quad[1], Quad[2], Quad[3]);
					U = HitUV.X;
				}
			}

			// Keep result that is furthest from the MinU
			const FReal Dist = FMath::Abs(U - EdgeU);
			if (Dist > BestDist)
			{
				BestDist = Dist;
				BestU = U;
				OutSegmentIndex = SegmentIndex;
			}
		}

		return BestU;
	}

	/**
	 * Turns soup of segments into a monotonic polyline (kinda like heightfield), removing segment intersections and overlaps.
	 * @param Segments segments to sample
	 * @param EdgeU U value of the edge we're currently processing (left = 0.0, right = 1.0)
	 * @param Quad bilinear patch used for mapping between world and UV space
	 * @param OutUVPoints Hit segment index, or INDEX_NONE if no hit.
	 */
	static void CalculateHullPolyline(const TConstArrayView<FUVSegment> Segments, const FVector::FReal EdgeU, const TConstArrayView<FVector> Quad, TArray<FVector2D>& OutUVPoints)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NavCorridor_CalculateHull);

		using FReal = FVector::FReal;

		// If there are no obstacle segments, just return a straight line of the edge.
		if (Segments.IsEmpty())
		{
			OutUVPoints.Add(FVector2D(EdgeU, 0.0));
			OutUVPoints.Add(FVector2D(EdgeU, 1.0));
			return;
		}

		// Collect all the potential locations of the final polyline. This include (unique) segment start and end locations and segment intersections.
		TArray<FReal> SampleLocations;
		SampleLocations.Add(0.0);	// We want exact start and end.
		SampleLocations.Add(1.0);

		for (int32 SegmentIndex = 0; SegmentIndex < Segments.Num(); SegmentIndex++)
		{
			const FUVSegment& CurrSegment = Segments[SegmentIndex];

			// Segment end points
			if (CurrSegment.StartUV.Y > 0.0 && CurrSegment.StartUV.Y < 1.0)
			{
				AddUnique(SampleLocations, CurrSegment.StartUV.Y);
			}
			if (CurrSegment.EndUV.Y > 0.0 && CurrSegment.EndUV.Y < 1.0)
			{
				AddUnique(SampleLocations, CurrSegment.EndUV.Y);
			}

			// Segment intersections.
			for (int32 NextSegmentIndex = SegmentIndex + 1; NextSegmentIndex < Segments.Num(); NextSegmentIndex++)
			{
				const FUVSegment& NextSegment = Segments[NextSegmentIndex];

				// Stop if the further segments cannot overlap in V direction.
				if (NextSegment.StartUV.Y >= CurrSegment.EndUV.Y)
				{
					break;
				}

				// If the segments are connected by end points, don't bother with intersection test.
				if (CurrSegment.EndUV.Equals(NextSegment.StartUV))
				{
					continue;
				}

				// If the segments overlap in V direction, check if they intersect.
				if (FMath::Max(CurrSegment.StartUV.Y, NextSegment.StartUV.Y) < FMath::Min(CurrSegment.StartUV.Y, NextSegment.StartUV.Y))
				{
					// @todo: profile if this is an issues and needs to be sped up. Should happen very rarely.
					// The segment intersection test is done in world space
					const FVector CurrentStartPos = UE::AI::Bilinear(CurrSegment.StartUV, Quad[0], Quad[1], Quad[2], Quad[3]);
					const FVector CurrentEndPos = UE::AI::Bilinear(CurrSegment.EndUV, Quad[0], Quad[1], Quad[2], Quad[3]);
					const FVector NextStartPos = UE::AI::Bilinear(NextSegment.StartUV, Quad[0], Quad[1], Quad[2], Quad[3]);
					const FVector NextEndPos = UE::AI::Bilinear(NextSegment.EndUV, Quad[0], Quad[1], Quad[2], Quad[3]);

					FReal CurrentT = 0.0, NextT = 0.0;
					if (UE::AI::IntersectLineLine2D(CurrentStartPos, CurrentEndPos, NextStartPos, NextEndPos, CurrentT, NextT))
					{
						const FVector IntersectionPos = FMath::Lerp(CurrentStartPos, CurrentEndPos, CurrentT);
						const FVector2D IntersectionUV = UE::AI::InvBilinear2DClamped(IntersectionPos, Quad[0], Quad[1], Quad[2], Quad[3]);
						if (IntersectionUV.Y > 0.0 && IntersectionUV.Y < 1.0)
						{
							AddUnique(SampleLocations, IntersectionUV.Y);
						}
					}
				}
			}
		}

		Algo::Sort(SampleLocations, [](const FReal A, const FReal B) { return A < B; });

		// Sample the segments at specified locations.
		int32 LastSegmentIndex = INDEX_NONE;  
		int32 LastLastSegmentIndex = INDEX_NONE;
		FSampleSegmentsIterator SampleIter;

		for (const FReal V : SampleLocations)
		{
			int32 SegmentIndex = INDEX_NONE;
			const FReal SampledU = SampleSegmentsHeight(SampleIter, V, EdgeU, Segments, Quad, SegmentIndex);
			const FVector2D UV(SampledU, V);

			// If a third point is sampled on the same segment, do not add new point, just move the end point.
			// This can happen for example for hidden end points of intersecting segments.
			if (OutUVPoints.Num() >= 2 && LastSegmentIndex == SegmentIndex && LastLastSegmentIndex == SegmentIndex)
			{
				OutUVPoints.Last() = UV;
			}
			else
			{
				OutUVPoints.Add(UV);
				LastLastSegmentIndex = LastSegmentIndex;
				LastSegmentIndex = SegmentIndex;
			}
		}
	}

	/** Simple iterator use to speed up consecutive interpolations. */
	struct FInterpolateIterator
	{
		int32 LastIndex = 1;
	};

	/**
	 * Interpolates monotonic polyline 'UVPoints' at location 'SampleAtV', and returns the location in world space.
	 * The curve values are interpolated in world space to make sure that the in between values are linear in world space.
	 * The functions is assumed to be called on multiple V locations in succession where each new V is larger than previous,
	 * FInterpolateIterator caches the progress.
	 * @param InOutIter interpolation progress
	 * @param SampleAtV location to sample 
	 * @param UVPoints UV curve
	 * @param Quad bilinear patch used for mapping between world and UV space
	 * @return interpolated location in world space.
	 */
	static FVector InterpolatePolyline(FInterpolateIterator& InOutIter, const FVector::FReal SampleAtV, const TConstArrayView<FVector2D> UVPoints, TConstArrayView<FVector> Quad)
	{
		using FReal = FVector::FReal;

		if (UVPoints.IsEmpty())
		{
			return FVector::Zero();
		}
	
		if (SampleAtV <= UVPoints[0].Y)
		{
			InOutIter.LastIndex = 1;
			return UE::AI::Bilinear(FVector2D(UVPoints[0].X, SampleAtV), Quad[0], Quad[1], Quad[2], Quad[3]);
		}

		for (int32 Index = InOutIter.LastIndex; Index < UVPoints.Num(); Index++)
		{
			const FVector2D& Prev = UVPoints[Index - 1];
			const FVector2D& Curr = UVPoints[Index];
			if (SampleAtV <= Curr.Y)
			{
				InOutIter.LastIndex = Index;
				
				const FReal Range = Curr.Y - Prev.Y;
				const FReal Diff = SampleAtV - Prev.Y;
				const FReal T = Range > UE_KINDA_SMALL_NUMBER ? (Diff /Range) : 0.0;

				// Do the interpolation in world space, so that we do not get the quad distortion. 
				const FVector VA = UE::AI::Bilinear(Prev, Quad[0], Quad[1], Quad[2], Quad[3]);
				const FVector VB = UE::AI::Bilinear(Curr, Quad[0], Quad[1], Quad[2], Quad[3]);

				return FMath::Lerp(VA, VB, T);
			}
		}

		InOutIter.LastIndex = UVPoints.Num();
		
		return UE::AI::Bilinear(FVector2D(UVPoints.Last().X, SampleAtV), Quad[0], Quad[1], Quad[2], Quad[3]);
	}

	/**
	 * Takes left and right hulls of the corridor in UV space, and merges them by creating a portal at every point location (either left or right hull).
	 * The original path segment gets samples on the portals too.
	 * A further pass will then simplify the portals to remove really small ones.
	 * @param LeftUVPoints left hull of the corridor 
	 * @param RightUVPoints right hull of the corridor
	 * @param Quad bilinear patch used for mapping between world and UV space
	 * @param PathSegmentStartPortal original path segment start portal
	 * @param PathSegmentEndPortal original path segment end portal
	 * @param PathPointBaseIndex original path segment start point index
	 * @param OutPortals resulting portals
	 */
	static void DivideHullsIntoPortals(const TConstArrayView<FVector2D> LeftUVPoints, const TConstArrayView<FVector2D> RightUVPoints, const TConstArrayView<FVector> Quad,
										const FNavCorridorPortal& PathSegmentStartPortal, const FNavCorridorPortal& PathSegmentEndPortal, const int32 PathPointBaseIndex,
										TArray<FNavCorridorPortal>& OutPortals)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NavCorridor_DivideHullsIntoPortals);

		using FReal = FVector::FReal;

		if (LeftUVPoints.IsEmpty() || RightUVPoints.IsEmpty())
		{
			return;
		}

		// Add unique V sampling locations from left and right hulls.
		TArray<FReal> SampleLocation;
		SampleLocation.Add(0.0);	// We want exact start and end.
		SampleLocation.Add(1.0);

		for (const FVector2D& Point : LeftUVPoints)
		{
			if (Point.Y > 0.0 && Point.Y < 1.0)
			{
				AddUnique(SampleLocation, Point.Y);
			}
		}
		for (const FVector2D& Point : RightUVPoints)
		{
			if (Point.Y > 0.0 && Point.Y < 1.0)
			{
				AddUnique(SampleLocation, Point.Y);
			}
		}
			
		Algo::Sort(SampleLocation, [](const FReal A, const FReal B) { return A < B; });

		// Only add the StartPortal on the first call to this function (OutPortals is empty) since the EndPortal will become the StartPortal for the next iteration.
		const bool bAddStartPortal = OutPortals.IsEmpty();

		FInterpolateIterator LeftIter;
		FInterpolateIterator RightIter;
		
		// Merge with previous portal if it exists, shrinking the portal as needed.
		if (OutPortals.Num() > 0)
		{
			FNavCorridorPortal& PrevPortal = OutPortals.Last();

			const FVector NewLeftPos = InterpolatePolyline(LeftIter, 0.0, LeftUVPoints, Quad);
			const FVector NewRightPos = InterpolatePolyline(RightIter, 0.0, RightUVPoints, Quad);

			const FReal LeftU = UE::AI::ProjectPointOnSegment2D(PrevPortal.Left, NewLeftPos, NewRightPos);
			const FReal RightU = UE::AI::ProjectPointOnSegment2D(PrevPortal.Right, NewLeftPos, NewRightPos);

			PrevPortal.Left = FMath::Lerp(NewLeftPos, NewRightPos, FMath::Max(0.0, LeftU));
			PrevPortal.Right = FMath::Lerp(NewLeftPos, NewRightPos, FMath::Min(RightU, 1.0));
		}

		// Interpolate and add rest of the portals.
		for (int32 DivIndex = bAddStartPortal ? 0 : 1; DivIndex < SampleLocation.Num(); DivIndex++)
		{
			const FReal V = SampleLocation[DivIndex];

			const FVector LeftPos = InterpolatePolyline(LeftIter, V, LeftUVPoints, Quad);
			const FVector RightPos = InterpolatePolyline(RightIter, V, RightUVPoints, Quad);

			FNavCorridorPortal& Portal = OutPortals.AddDefaulted_GetRef();
			Portal.Left = LeftPos;
			Portal.Right = RightPos;
			Portal.Location = FMath::Lerp(PathSegmentStartPortal.Location, PathSegmentEndPortal.Location, V);
			Portal.PathPointIndex = (int16)PathPointBaseIndex;
		}

		if (bAddStartPortal)
		{
			FNavCorridorPortal& FirstPortal = OutPortals[0];
			FirstPortal.bIsPathCorner = true;
			FirstPortal.bIsPinnedLocation = PathSegmentStartPortal.bIsPinnedLocation;
		}

		FNavCorridorPortal& LastPortal = OutPortals.Last();
		LastPortal.bIsPathCorner = true;
		LastPortal.bIsPinnedLocation = PathSegmentEndPortal.bIsPinnedLocation;
	}

	/**
	 * Simplifies a commonly occurring pattern where a small sector is
	 * Surrounded by almost straight boundary at either side.
	 * Straightness is controlled using the StraightThreshold.
	 * The simplification looks almost like flipping the portal.
	 * Only really small sectors are removed as the process can distort the corridor. 
	 *
	 *  0    1 2   3
	 *  o----o-o_           o------o_
	 *  :    : : ¨-o        :      : ¨-o
	 *  :    : :   :   =>   :     :    :
	 *  o-_  : :   :        o-_  .     :
	 *     ¨-o-o---o           ¨-o-----o
	 *
	 * @param Portals corridor portals to simplify 
	 * @param StraightThreshold 3 points are considered straight when they are this close from the line between them.
	 */
	static void SimplifyFlipPortals(TArray<FNavCorridorPortal>& Portals, const FVector::FReal StraightThreshold)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NavCorridor_SimplifyFlipPortals);

		using FReal = FVector::FReal;

		const FReal SmallDistanceThreshold = StraightThreshold * 0.5;
		
		for (int32 Index = 1; Index < Portals.Num() - 2; Index++)
		{
			const FNavCorridorPortal& Portal0 = Portals[Index - 1];
			const FNavCorridorPortal& Portal1 = Portals[Index];
			const FNavCorridorPortal& Portal2 = Portals[Index + 1];
			const FNavCorridorPortal& Portal3 = Portals[Index + 2];

			// Skip simplification for pinned locations, they need to remain in the final list.
			if (Portal2.bIsPinnedLocation)
			{
				continue;
			}

			// Only try to remove short sectors.
			const FReal ApproxPortalDistance = ApproxDistanceSegmentSegment(Portal1.Left, Portal1.Right, Portal2.Left, Portal2.Right);
			if (ApproxPortalDistance < SmallDistanceThreshold)
			{
				// Negative distance is outside (convex).
				const FReal LeftDist1 = UE::AI::SignedDistancePointLine2D(Portal1.Left, Portal0.Left, Portal2.Left);
				const FReal LeftDist2 = UE::AI::SignedDistancePointLine2D(Portal2.Left, Portal1.Left, Portal3.Left);
				const FReal RightDist1 = -UE::AI::SignedDistancePointLine2D(Portal1.Right, Portal0.Right, Portal2.Right);
				const FReal RightDist2 = -UE::AI::SignedDistancePointLine2D(Portal2.Right, Portal1.Right, Portal3.Right);

				const bool bLeftConvex1 = LeftDist1 < ConvexEpsilon;
				const bool bLeftConvex2 = LeftDist2 < ConvexEpsilon;
				
				const bool bRightConvex1 = RightDist1 < ConvexEpsilon;
				const bool bRightConvex2 = RightDist2 < ConvexEpsilon;

				const bool bLeftStraight1 = LeftDist1 > -StraightThreshold;
				const bool bLeftStraight2 = LeftDist2 > -StraightThreshold;
				
				const bool bRightStraight1 = RightDist1 > -StraightThreshold;
				const bool bRightStraight2 = RightDist2 > -StraightThreshold;

				bool bCanRemoveLeft = (bLeftConvex1 && bLeftStraight1) && (bRightConvex2 && bRightStraight2);
				bool bCanRemoveRight = (bLeftConvex2 && bLeftStraight2) && (bRightConvex1 && bRightStraight1);

				if (bCanRemoveLeft && bCanRemoveRight)
				{
					if ((LeftDist1 + RightDist2) < (LeftDist2 + RightDist1))
					{
						bCanRemoveRight = false;
					}
					else
					{
						bCanRemoveLeft = false;
					}
				}
				
				if (bCanRemoveLeft)
				{
					Portals[Index].Left = Portals[Index + 1].Left;
					Portals[Index].bIsPathCorner |= Portals[Index + 1].bIsPathCorner;
					Portals.RemoveAt(Index+1);
				}
				else if (bCanRemoveRight)
				{
					Portals[Index].Right = Portals[Index + 1].Right;
					Portals[Index].bIsPathCorner |= Portals[Index + 1].bIsPathCorner;
					Portals.RemoveAt(Index+1);
				}
			}
		}
	}

	/**
	 * Simplifies corridor by removing portals that are between two sectors whose shared edges are almost straight,
	 * and convex around the middle portal.
	 * Skips merging next to large sectors, since that can lead to big corridor volume loss.
	 *
	 *  0    1   2
	 *  o----o---o        o--------o
	 *  :    :   :        :        :
	 *  :    :   :   =>   :        :
	 *  :    :   :        :        :
	 *  o--__o__-o        o--------o
	 *
	 * @param Portals corridor portals to simplify 
	 * @param StraightThreshold 3 points are considered straight when they are this close from the line between them.
	 * @param SmallSectorThreshold a sector is considered to bet merged if it's shorter than this
	 * @param LargeSectorThreshold simplification is skipped if one of the sectors is longer than this
	 */
	static void SimplifyConvexPortals(TArray<FNavCorridorPortal>& Portals, const FVector::FReal StraightThreshold, const FVector::FReal SmallSectorThreshold, const FVector::FReal LargeSectorThreshold)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NavCorridor_SimplifyConvexPortals);

		using FReal = FVector::FReal;
		
		for (int32 Index = 1; Index < Portals.Num() - 1; Index++)
		{
			FNavCorridorPortal& Portal0 = Portals[Index - 1];
			const FNavCorridorPortal& Portal1 = Portals[Index];
			FNavCorridorPortal& Portal2 = Portals[Index + 1];

			// Skip simplification for pinned locations, they need to remain in the final list.
			if (Portal1.bIsPinnedLocation)
			{
				continue;
			}

			const FReal ApproxPortalDistance02 = ApproxDistanceSegmentSegment(Portal0.Left, Portal0.Right, Portal2.Left, Portal2.Right);
			if (ApproxPortalDistance02 > LargeSectorThreshold)
			{
				continue;
			}

			// Negative distance is outside (convex).
			const FReal LeftDist = UE::AI::SignedDistancePointLine2D(Portal1.Left, Portal0.Left, Portal2.Left);
			const FReal RightDist = -UE::AI::SignedDistancePointLine2D(Portal1.Right, Portal0.Right, Portal2.Right);

			const bool bLeftInBounds = LeftDist > -StraightThreshold;
			const bool bRightInBounds = RightDist > -StraightThreshold; 

			const bool bLeftConvex = LeftDist < ConvexEpsilon;
			const bool bRightConvex = RightDist < ConvexEpsilon;

			if (bLeftConvex && bRightConvex)
			{
				if ((ApproxPortalDistance02 < SmallSectorThreshold) || (bLeftInBounds && bRightInBounds))
				{
					// Merge corner flag
					if (Portal1.bIsPathCorner)
					{
						const FReal ApproxPortalDistance01 = ApproxDistanceSegmentSegment(Portal0.Left, Portal0.Right, Portal1.Left, Portal1.Right);
						const FReal ApproxPortalDistance12 = ApproxDistanceSegmentSegment(Portal1.Left, Portal1.Right, Portal2.Left, Portal2.Right);
						if (ApproxPortalDistance01 < ApproxPortalDistance12)
						{
							Portal0.bIsPathCorner |= Portal1.bIsPathCorner; 
						}
						else
						{
							Portal2.bIsPathCorner |= Portal1.bIsPathCorner; 
						}
					}
					
					Portals.RemoveAt(Index);
					Index--;
				}
			}
		}
	}

	/**
	 * Simplifies corridor by removing portals that are between two sectors whose shared edges are almost straight,
	 * and concave around the middle portal. The adjacent portals and shrunk to ensure that the new sector will not
	 * spill into non-navigable space. 
	 *
	 *  0    1   2
	 *  o----o---o        o--------o
	 *  :    :   :        :        :
	 *  :    :   :   =>   :        :
	 *  :  _-o_  :        o--------o
	 *  o-¨    ¨-o
	 *
	 * @param Portals corridor portals to simplify 
	 * @param StraightThreshold 3 points are considered straight when they are this close from the line between them.
	 * @param LargeSectorThreshold simplification is skipped if one of the sectors is longer than this
	 */
	static void SimplifyConcavePortals(TArray<FNavCorridorPortal>& Portals, const FVector::FReal StraightThreshold, const FVector::FReal LargeSectorThreshold)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NavCorridor_SimplifyConcavePortals);

		using FReal = FVector::FReal;

		const FReal ConcaveStraightThreshold = StraightThreshold * 0.5;
		
		for (int32 Index = 1; Index < Portals.Num() - 1; Index++)
		{
			FNavCorridorPortal& Portal0 = Portals[Index - 1];
			FNavCorridorPortal& Portal1 = Portals[Index];
			FNavCorridorPortal& Portal2 = Portals[Index + 1];

			// Skip simplification for pinned locations, they need to remain in the final list.
			if (Portal1.bIsPinnedLocation)
			{
				continue;
			}

			const FReal ApproxPortalDistance02 = ApproxDistanceSegmentSegment(Portal0.Left, Portal0.Right, Portal2.Left, Portal2.Right);
			if (ApproxPortalDistance02 > LargeSectorThreshold)
			{
				continue;
			}

			const FReal LeftDist = UE::AI::SignedDistancePointLine2D(Portal1.Left, Portal0.Left, Portal2.Left);
			const FReal RightDist = -UE::AI::SignedDistancePointLine2D(Portal1.Right, Portal0.Right, Portal2.Right);

			const bool bLeftConvex = LeftDist < ConvexEpsilon;
			const bool bRightConvex = RightDist < ConvexEpsilon;

			// @todo: we should consider the options too where the simplification direction is along the longer segment
			// @todo: similarly, some vertices (e.g. a path corner) are more important to retain than others

			if (bLeftConvex || bRightConvex)
			{
				const bool bCanFlattenLeft = !bLeftConvex && bRightConvex && LeftDist < ConcaveStraightThreshold && RightDist > -StraightThreshold;
				const bool bCanFlattenRight = !bRightConvex && bLeftConvex && RightDist < ConcaveStraightThreshold && LeftDist > -StraightThreshold;
				
				if (bCanFlattenLeft || bCanFlattenRight)
				{
					if (bCanFlattenLeft)
					{
						// @todo: check that the adjusted portal does not flip or collapse.
						FReal Temp = 0.0, T0 = 0.0, T2 = 0.0;
						const FVector Dir = Portal2.Left - Portal0.Left;
						UE::AI::IntersectLineLine2D(Portal1.Left, Portal1.Left + Dir, Portal0.Left, Portal0.Right, Temp, T0);
						UE::AI::IntersectLineLine2D(Portal1.Left, Portal1.Left + Dir, Portal2.Left, Portal2.Right, Temp, T2);

						if (bUseSimplifyConcavePortalsFix && (T0 < 0 || T2 < 0 || T0 > 1 || T2 > 1))
						{
							// Skip simplify this portal to avoid degenerating the path.
							continue;
						}

						const FVector AdjustedLeft0 = FMath::Lerp(Portal0.Left, Portal0.Right, T0);
						const FVector AdjustedLeft2 = FMath::Lerp(Portal2.Left, Portal2.Right, T2);

						Portal0.Left = AdjustedLeft0;
						Portal2.Left = AdjustedLeft2;
					}
					else
					{
						// @todo: check that the adjusted portal does not flip or collapse.
						FReal Temp = 0.0, T0 = 0.0, T2 = 0.0;
						const FVector Dir = Portal2.Right - Portal0.Right;
						UE::AI::IntersectLineLine2D(Portal1.Right, Portal1.Right + Dir, Portal0.Left, Portal0.Right, Temp, T0);
						UE::AI::IntersectLineLine2D(Portal1.Right, Portal1.Right + Dir, Portal2.Left, Portal2.Right, Temp, T2);

						if (bUseSimplifyConcavePortalsFix && (T0 < 0 || T2 < 0 || T0 > 1 || T2 > 1))
						{
							// Skip simplify this portal to avoid degenerating the path.
							continue;
						}

						const FVector AdjustedRight0 = FMath::Lerp(Portal0.Left, Portal0.Right, T0);
						const FVector AdjustedRight2 = FMath::Lerp(Portal2.Left, Portal2.Right, T2);

						Portal0.Right = AdjustedRight0;
						Portal2.Right = AdjustedRight2;
					}

					// Merge corner flag
					if (Portal1.bIsPathCorner)
					{
						const FReal ApproxPortalDistance01 = ApproxDistanceSegmentSegment(Portal0.Left, Portal0.Right, Portal1.Left, Portal1.Right);
						const FReal ApproxPortalDistance12 = ApproxDistanceSegmentSegment(Portal1.Left, Portal1.Right, Portal2.Left, Portal2.Right);
						if (ApproxPortalDistance01 < ApproxPortalDistance12)
						{
							Portal0.bIsPathCorner |= Portal1.bIsPathCorner;
						}
						else
						{
							Portal2.bIsPathCorner |= Portal1.bIsPathCorner;
						}
					}

					Portals.RemoveAt(Index);
				}
			}
		}
	}

	/**
	 * Interpolates path location on portals between given portal indices.
	 * Start and end points will stay in place, and all the points in between will be interpolated along a lime.
	 * @param Portals portals defining the corridor
	 * @param StartIndex start index of the interpolation
	 * @param EndIndex end index of the interpolation
	 */
	static void InterpolateInBetweenLocations(TArrayView<FNavCorridorPortal> Portals, const int32 StartIndex, const int32 EndIndex)
	{
		using FReal = FVector::FReal;
		
		const FVector PrevLocation = Portals[StartIndex].Location;
		const FVector CurrLocation = Portals[EndIndex].Location;
					
		for (int32 MidIndex = StartIndex + 1; MidIndex < EndIndex; MidIndex++)
		{
			FNavCorridorPortal& MidPortal = Portals[MidIndex];
			FReal SegT, MidPortalT;
			if (UE::AI::IntersectLineLine2D(PrevLocation, CurrLocation, MidPortal.Left, MidPortal.Right, SegT, MidPortalT))
			{
				MidPortal.Location = FMath::Lerp(MidPortal.Left, MidPortal.Right, FMath::Clamp(MidPortalT, 0.0f, 1.0f));
			}
		}
	}

	/**
	 * Returns a portal's extremities pulled in from their original position with OffsetWidth. If the portal is
	 * smaller than 2x width, the center point will be used
	 * @param Portal The portal to get Left/Right from
	 * @param OffsetWidth How much to offset from the original corridor boundaries
	 * @param OutOffsettedLeft The offsetted left extremity of the portal
	 * @param OutOffsettedRight The offsetted right extremity of the portal
	 */
	static void GetPortalLeftRightWithOffset(const FNavCorridorPortal& Portal, const float OffsetWidth, FVector& OutOffsettedLeft, FVector& OutOffsettedRight)
	{
		using FReal = FVector::FReal;

		if (OffsetWidth <= UE_KINDA_SMALL_NUMBER)
		{
			OutOffsettedLeft = Portal.Left;
			OutOffsettedRight = Portal.Right;
			return;
		}

		const FVector PortalDirection = Portal.Right - Portal.Left;
		const FReal PortalWidthSquared = PortalDirection.SizeSquared2D();
		const FReal TwoOffsetSquared = (2.0f * OffsetWidth) * (2.0f * OffsetWidth);
		if (PortalWidthSquared < TwoOffsetSquared)
		{
			const FVector Middle = Portal.Left + 0.5 * PortalDirection;
			OutOffsettedLeft = Middle;
			OutOffsettedRight = Middle;
		}
		else
		{
			const FReal PortalWidth = FMath::Sqrt(PortalWidthSquared);
			const FVector Offset = (PortalDirection / PortalWidth) * OffsetWidth; // Assuming the portal is horizontal enough to ignore offset width that gets consumed along the Z axis
			OutOffsettedLeft = Portal.Left + Offset;
			OutOffsettedRight = Portal.Right - Offset;
		}
	}

	/**
	 * Pulls string between path locations on given portal indices.
	 * Start and end points will stay in place, and all the points in between will be on shortest path along the corridor.
	 * @param Portals portals defining the corridor
	 * @param StartIndex start index of the string pull
	 * @param EndIndex end index of the string pull
	 * @param OffsetFromBoundaries optional offset that will push the path away from the portal extremities and towards the center. No effect when set to 0.0
	 */
	static void StringPull(TArrayView<FNavCorridorPortal> Portals, const int32 StartIndex, const int32 EndIndex, const float OffsetFromBoundaries)
	{
		check(Portals.IsValidIndex(StartIndex));
		check(Portals.IsValidIndex(EndIndex));
		check(StartIndex <= EndIndex);
		
		FVector PortalApex = Portals[StartIndex].Location;
		FVector PortalLeft;
		FVector PortalRight;
		GetPortalLeftRightWithOffset(Portals[StartIndex], OffsetFromBoundaries, PortalLeft, PortalRight);

		int32 ApexIndex = StartIndex;
		int32 LeftIndex = StartIndex;
		int32 RightIndex = StartIndex;
		
		for (int32 Index = StartIndex + 1; Index <= EndIndex; Index++)
		{
			const FNavCorridorPortal& CurrPortal = Portals[Index];
			FVector Left;
			FVector Right;
			if (Index == EndIndex || CurrPortal.bIsPinnedLocation)
			{
				Left = CurrPortal.Location;
				Right = CurrPortal.Location;
			}
			else
			{
				GetPortalLeftRightWithOffset(Portals[Index], OffsetFromBoundaries, Left, Right);
			}

			if (UE::AI::TriArea2D(PortalApex, PortalRight, Right) >= 0.0)
			{
				if (PortalApex.Equals(PortalRight) || UE::AI::TriArea2D(PortalApex, PortalLeft, Right) < 0.0)
				{
					// Tighten the funnel
					PortalRight = Right;
					RightIndex = Index;
				}
				else
				{
					// Right over left, insert left to path and restart scan from portal left point.
					Portals[LeftIndex].Location = PortalLeft;
					Portals[LeftIndex].bIsPathCorner = true;
					InterpolateInBetweenLocations(Portals, ApexIndex, LeftIndex);
					// Make current left the new apex.
					PortalApex = PortalLeft;
					ApexIndex = LeftIndex;
					// Reset portal
					PortalLeft = PortalApex;
					PortalRight = PortalApex;
					LeftIndex = ApexIndex;
					RightIndex = ApexIndex;
					// Restart scan
					Index = ApexIndex;
					continue;
				}
			}
			if (UE::AI::TriArea2D(PortalApex, PortalLeft, Left) <= 0.0)
			{
				if (PortalApex.Equals(PortalLeft) || UE::AI::TriArea2D(PortalApex, PortalRight, Left) > 0.0)
				{
					// Tighten the funnel
					PortalLeft = Left;
					LeftIndex = Index;
				}
				else
				{
					// Left over right, insert right to path and restart scan from portal right point.
					Portals[RightIndex].Location = PortalRight;
					Portals[RightIndex].bIsPathCorner = true;
					InterpolateInBetweenLocations(Portals, ApexIndex, RightIndex);
					// Make current left the new apex.
					PortalApex = PortalRight;
					ApexIndex = RightIndex;
					// Reset portal
					PortalLeft = PortalApex;
					PortalRight = PortalApex;
					LeftIndex = ApexIndex;
					RightIndex = ApexIndex;
					// Restart scan
					Index = ApexIndex;
					continue;
				}
			}
		}
	}

	/** Projects the 'Location' of the portal on the portal segment. */
	static void ProjectLocationOnPortal(FNavCorridorPortal& InOutPortal)
	{
		const double LocationU = UE::AI::ProjectPointOnSegment2D(InOutPortal.Location, InOutPortal.Left, InOutPortal.Right);
		InOutPortal.Location = FMath::Lerp(InOutPortal.Left, InOutPortal.Right, LocationU);
	}

	/**
	 * Finds how much the end portal is visible from the start portal path location.
	 * @param Portals portals defining the corridor
	 * @param StartIndex start index of the visibility check
	 * @param EndIndex end index of the interpolation
	 * @return Visibility min and max value in range [0..1] on the end portal between left and right portal points.  
	 */
	static FVector2D FindForwardVisibility(TArrayView<FNavCorridorPortal> Portals, const int32 StartIndex, const int32 EndIndex)
	{
		using FReal = FVector::FReal;

		const FVector Apex = Portals[StartIndex].Location;
		FVector Left = Portals[StartIndex].Left;
		FVector Right = Portals[StartIndex].Right;
		for (int32 Index = StartIndex + 1; Index <= EndIndex; Index++)
		{
			FNavCorridorPortal& CurrPortal = Portals[Index];
			if (UE::AI::TriArea2D(Apex, Right, CurrPortal.Right) >= 0.0)
			{
				Right = CurrPortal.Right;
			}

			if (UE::AI::TriArea2D(Apex, Left, CurrPortal.Left) <= 0.0)
			{
				Left = CurrPortal.Left;
			}
		}

		const FNavCorridorPortal& EndPortal = Portals[EndIndex]; 
		
		FVector2D Result(0.0, 1.0);
		FReal TA, TB;
		
		if (UE::AI::IntersectLineLine2D(EndPortal.Left, EndPortal.Right, Apex, Left, TA, TB))
		{
			Result.X = FMath::Clamp(TA, 0.0, 1.0);
		}
		
		if (UE::AI::IntersectLineLine2D(EndPortal.Left, EndPortal.Right, Apex, Right, TA, TB))
		{
			Result.Y = FMath::Clamp(TA, 0.0, 1.0);
		}

		return Result;
	}

	/**
	 * Finds how much the start portal is visible from the end portal path location.
	 * @param Portals portals defining the corridor
	 * @param StartIndex start index of the visibility check
	 * @param EndIndex end index of the interpolation
	 * @return Visibility min and max value in range [0..1] on the start portal between left and right portal points.  
	 */
	static FVector2D FindBackwardVisibility(TArrayView<FNavCorridorPortal> Portals, const int32 StartIndex, const int32 EndIndex)
	{
		using FReal = FVector::FReal;

		// Backward
		const FVector Apex = Portals[EndIndex].Location;
		FVector Left = Portals[EndIndex].Left;
		FVector Right = Portals[EndIndex].Right;
		for (int32 Index = EndIndex - 1; Index >= StartIndex; Index--)
		{
			FNavCorridorPortal& CurrPortal = Portals[Index];
			if (UE::AI::TriArea2D(Apex, Right, CurrPortal.Right) <= 0.0)
			{
				Right = CurrPortal.Right;
			}

			if (UE::AI::TriArea2D(Apex, Left, CurrPortal.Left) >= 0.0)
			{
				Left = CurrPortal.Left;
			}
		}

		const FNavCorridorPortal& StartPortal = Portals[StartIndex]; 
		
		FVector2D Result(0.0, 1.0);
		FReal TA, TB;
		
		if (UE::AI::IntersectLineLine2D(StartPortal.Left, StartPortal.Right, Apex, Left, TA, TB))
		{
			Result.X = FMath::Clamp(TA, 0.0, 1.0);
		}
		
		if (UE::AI::IntersectLineLine2D(StartPortal.Left, StartPortal.Right, Apex, Right, TA, TB))
		{
			Result.Y = FMath::Clamp(TA, 0.0, 1.0);
		}

		return Result;
	}

	/**
	 * Finds the next portal marked as corner starting from specified index.
	 * @param Portals portals defining the corridor
	 * @param StartIndex start index of the query
	 * @return Index of next corner points, or INDEX_NONE if not found.   
	 */
	static int32 FindNextCorner(TArrayView<FNavCorridorPortal> Portals, const int32 StartIndex)
	{
		for (int32 Index = StartIndex + 1; Index < Portals.Num(); Index++)
		{
			if (Portals[Index].bIsPathCorner)
			{
				return Index;
			}
		}
		return INDEX_NONE;
	}

}; // UE::NavCorridor::Private


//-------------------------------------------------------
// FNavCorridorParams
//-------------------------------------------------------

FString FNavCorridorParams::ToString() const
{
	return FString::Printf(TEXT("Width: %.1f, PathOffsetFromBoundaries: %.1f, ObstacleTaperAngle: %.1f, SmallSectorThreshold %.1f, " 
		"LargeSectorThreshold: %.1f, SimplifyEdgeThreshold: %.1f, bSimplifyFlipPortals: %i, bSimplifyConvexPortals: %i, bSimplifyConcavePortals: %i"),
		Width,
		PathOffsetFromBoundaries,
		ObstacleTaperAngle,
		SmallSectorThreshold,
		LargeSectorThreshold,
		SimplifyEdgeThreshold,
		bSimplifyFlipPortals,
		bSimplifyConvexPortals,
		bSimplifyConcavePortals);
}


//-------------------------------------------------------
// FNavCorridor
//-------------------------------------------------------
const FVector FNavCorridor::OverlappingPointTolerance = FVector(1,1,30); 

void FNavCorridor::Reset()
{
	Portals.Reset();
}

void FNavCorridor::BuildFromPath(const FNavigationPath& Path, FSharedConstNavQueryFilter NavQueryFilter, const FNavCorridorParams& Params)
{
	if (Path.GetPathPoints().Num() < 2)
	{
		Reset();
		return;
	}
	
	BuildFromPathPoints(Path, Path.GetPathPoints(), 0, NavQueryFilter, Params);
}

void FNavCorridor::BuildFromPathPoints(const FNavigationPath& Path, TConstArrayView<FNavPathPoint> PathPoints, const int32 PathPointBaseIndex,
										FSharedConstNavQueryFilter NavQueryFilter, const FNavCorridorParams& Params)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NavCorridor_BuildFromPath);

	if (PathPoints.Num() < 2)
	{
		Reset();
		return;
	}
	
	const ANavigationData* NavData = Path.GetNavigationDataUsed();
	check(NavData);

	using FReal = FVector::FReal;

	Reset(); 

	// First calculate wide corridor from the path segments.
	TArray<FVector> SegmentDirs;
	TArray<FVector> SegmentLefts;
	TArray<FNavCorridorPortal> InitialPortals;

	SegmentDirs.Reserve(PathPoints.Num() - 1);
	SegmentLefts.Reserve(PathPoints.Num() - 1);
	InitialPortals.Reserve(PathPoints.Num());

	UE::NavCorridor::Private::CalculateSegmentDirections(PathPoints, SegmentDirs, SegmentLefts);
	UE::NavCorridor::Private::CalculateCorridorPortals(PathPoints, Params.PinnedPointsIndices, SegmentLefts, Params.Width, InitialPortals);

	const FReal TaperLength = FMath::Tan(FMath::DegreesToRadians(Params.ObstacleTaperAngle)) * Params.Width;
	
	check(InitialPortals.Num() == PathPoints.Num());

	TArray<FVector> Edges;
	Edges.Reserve(20); // Usually there are less than 10 segments. 

	const UObject* PathQuerier = Path.GetQuerier();
	for (int32 PortalIndex = 0; PortalIndex < InitialPortals.Num() - 1; PortalIndex++)
	{
		FNavCorridorPortal& CurrPortal = InitialPortals[PortalIndex];
		FNavCorridorPortal& NextPortal = InitialPortals[PortalIndex + 1];

		// Quad describing the sector between two portals.
		// The sector quad is guaranteed to be convex by CalculateCorridorPortals().
		TArray<FVector, TInlineAllocator<4>> Quad;
		Quad.Add(CurrPortal.Left);
		Quad.Add(CurrPortal.Right);
		Quad.Add(NextPortal.Right);
		Quad.Add(NextPortal.Left);

		// Find wall/obstacle segments that overlap the current sector between two portals.
		{
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NavCorridor_FindOverlappingEdges);

			Edges.Reset();
			NavData->GetPathSegmentBoundaryEdges(Path,  PathPoints[PortalIndex], PathPoints[PortalIndex+1], Quad, Edges, 0.1f, NavQueryFilter);

#if ENABLE_VISUAL_LOG
			if (UE::NavCorridor::Private::bEnableNavCorridorVLog)
			{
				const FColor RandomColor = FLinearColor::MakeFromHSV8(FRandomStream(PortalIndex).FRandRange(0, 255.0), 196, 196).ToFColor(false).WithAlpha(128);
				UE_VLOG_CONVEXPOLY(PathQuerier, LogNavCorridorQuads, Log, TArray<FVector>(Quad), RandomColor, TEXT_EMPTY);
				for (int32 Index = 0; Index < Edges.Num(); Index += 2)
				{
					const FVector PolyOffset(0, 0, 15);
					UE_VLOG_ARROW(PathQuerier, LogNavCorridorEdges, Log, Edges[Index] + PolyOffset, Edges[Index + 1] + PolyOffset, FColor::Black, TEXT_EMPTY);
				}
			}
#endif // ENABLE_VISUAL_LOG

		}

		// Clip and classify edges to left and right segments.
		// The resulting segments are in the sector quads parametric space. This ensures that there's good continuity between the sectors.
		TArray<UE::NavCorridor::Private::FUVSegment> LeftUVSegments;
		TArray<UE::NavCorridor::Private::FUVSegment> RightUVSegments;

		const FVector SegBasePos = PathPoints[PortalIndex].Location;
		const FVector SegLeft = SegmentLefts[PortalIndex];
		const FVector SegDir = SegmentDirs[PortalIndex];

		UE::NavCorridor::Private::ClipEdgesToSector(PathQuerier, SegBasePos, SegDir, SegLeft, Quad, Edges, LeftUVSegments, RightUVSegments);

		// Process obstacle segments so that they do not have steep angles, by adding taper segments at obstacle ends.
		TArray<FVector2d> LeftUVPoints;
		TArray<FVector2d> RightUVPoints;

		const FReal LeftSideDistance = FVector::Distance(CurrPortal.Left, NextPortal.Left);
		const FReal RightSideDistance = FVector::Distance(CurrPortal.Right, NextPortal.Right);
		const FReal LeftSlopeOffsetV = LeftSideDistance > UE_KINDA_SMALL_NUMBER ? (TaperLength / LeftSideDistance) : 0.0;
		const FReal RightSlopeOffsetV = RightSideDistance > UE_KINDA_SMALL_NUMBER ? (TaperLength / RightSideDistance) : 0.0;

		UE::NavCorridor::Private::AddTaperSegments(LeftUVSegments, 0.0, LeftSlopeOffsetV);
		UE::NavCorridor::Private::AddTaperSegments(RightUVSegments, 1.0, RightSlopeOffsetV);

		// Calculate monotonic polyline hull of each side. 
		UE::NavCorridor::Private::CalculateHullPolyline(LeftUVSegments, 0.0, Quad, LeftUVPoints);
		UE::NavCorridor::Private::CalculateHullPolyline(RightUVSegments, 1.0, Quad, RightUVPoints);
		
		// Combine the hulls and create portals at each vertex on left and right hull.
		UE::NavCorridor::Private::DivideHullsIntoPortals(LeftUVPoints, RightUVPoints, Quad, CurrPortal, NextPortal, PathPointBaseIndex + PortalIndex, Portals);
	}

	// The process above can leave short sectors in the corridor. The passes below remove common patterns of small sectors.
	if (Params.bSimplifyFlipPortals)
	{
		UE::NavCorridor::Private::SimplifyFlipPortals(Portals, Params.SimplifyEdgeThreshold);
	}
	if (Params.bSimplifyConvexPortals)
	{
		UE::NavCorridor::Private::SimplifyConvexPortals(Portals, Params.SimplifyEdgeThreshold, Params.SmallSectorThreshold, Params.LargeSectorThreshold);
	}
	if (Params.bSimplifyConcavePortals)
	{
		UE::NavCorridor::Private::SimplifyConcavePortals(Portals, Params.SimplifyEdgeThreshold, Params.LargeSectorThreshold);
	}

	if (Portals.Num() > 1)
	{
		// Ensure that the start and end locations are on portal
		UE::NavCorridor::Private::ProjectLocationOnPortal(Portals[0]);
		UE::NavCorridor::Private::ProjectLocationOnPortal(Portals.Last());
		
		// String pull the path again, might not be correct anymore after all the processing.
		if (Portals.Num() > 2)
		{
			// String pull
			UE::NavCorridor::Private::StringPull(Portals, 0, Portals.Num() - 1, Params.PathOffsetFromBoundaries);
		}
	}

#if ENABLE_VISUAL_LOG
	if (UE::NavCorridor::Private::bEnableNavCorridorVLog)
	{
		if (const INavAgentInterface* PathQuerierNavAgent = Cast<const INavAgentInterface>(PathQuerier))
		{
			UE_VLOG_LOCATION(PathQuerier, LogNavCorridorPath, Log, PathQuerierNavAgent->GetNavAgentLocation(), 20.f, FColor::Green, TEXT_EMPTY);
		}

		// Draw path
		for (int32 PointIndex = 0; PointIndex < PathPoints.Num(); PointIndex++)
		{
			if ((PointIndex + 1) < PathPoints.Num())
			{
				UE_VLOG_SEGMENT_THICK(PathQuerier, LogNavCorridorPath, Log, PathPoints[PointIndex].Location, PathPoints[PointIndex + 1].Location, FColor(64, 64, 64), 2, TEXT_EMPTY);
			}

			UE_VLOG_LOCATION(PathQuerier, LogNavCorridorPath, Log, PathPoints[PointIndex].Location, 20.f, FColor::White, TEXT("%d-%d%s"), PointIndex, FNavMeshNodeFlags(PathPoints[PointIndex].Flags).AreaFlags, Params.PinnedPointsIndices.Contains(PointIndex) ? TEXT(" Pinned") : TEXT_EMPTY);
		}

		const FVector Offset(0.f, 0.f, 10.f);
		for (int32 PortalIndex = 0; PortalIndex < Portals.Num(); PortalIndex++)
		{
			const FNavCorridorPortal& Portal = Portals[PortalIndex];

			// Portal segment
			UE_VLOG_SEGMENT_THICK(PathQuerier, LogNavCorridorPath, Log, Offset + Portal.Left, Offset + Portal.Right, Portal.bIsPathCorner ? FColor::Red : FColor::Orange, Portal.bIsPathCorner ? 2 : 1, TEXT_EMPTY);

			if ((PortalIndex + 1) < Portals.Num())
			{
				// Sector boundaries
				const FNavCorridorPortal& NextPortal = Portals[PortalIndex + 1];
				UE_VLOG_SEGMENT_THICK(PathQuerier, LogNavCorridorPath, Log, Offset + Portal.Left, Offset + NextPortal.Left, FColor::Orange, 2, TEXT_EMPTY);
				UE_VLOG_SEGMENT_THICK(PathQuerier, LogNavCorridorPath, Log, Offset + Portal.Right, Offset + NextPortal.Right, FColor::Orange, 2, TEXT_EMPTY);

				// Path
				UE_VLOG_SEGMENT_THICK(PathQuerier, LogNavCorridorPath, Log, Offset + Portal.Location, Offset + NextPortal.Location, FColor(255, 128, 128), 2, TEXT_EMPTY);
			}

			UE_VLOG_LOCATION(PathQuerier, LogNavCorridorPath, Log, Offset + Portal.Location, 5.f, Portal.bIsPinnedLocation ? FColor(75, 75, 75) : FColor::White, TEXT("%d%s"), PortalIndex, Portal.bIsPinnedLocation ? TEXT(" (Pinned)" : TEXT_EMPTY));
		}
	}
#endif // ENABLE_VISUAL_LOG

}

void FNavCorridor::OffsetPathLocationsFromWalls(const float Offset, bool bOffsetFirst, bool bOffsetLast)
{
	using FReal = FVector::FReal;

	int32 PrevCornerIndex = INDEX_NONE;

	for (int32 PortalIndex = 0; PortalIndex < Portals.Num(); PortalIndex++)
	{
		FNavCorridorPortal& CurrPortal = Portals[PortalIndex];
		if (CurrPortal.bIsPathCorner)
		{
			bool bCanOffset = true;
			if (PortalIndex == 0)
			{
				bCanOffset = bOffsetFirst;
			}
			else if (PortalIndex == (Portals.Num() - 1))
			{
				bCanOffset = bOffsetLast;
			}
			
			if (bCanOffset)
			{
				//               o
				//     _o_     _¨ N
				//  o-¨ : ¨-o-¨    _o
				//  P   :   X   _-¨
				//  o---o----o-¨
				//
				//  A corner portals is a portal which is placed at the location of original path point.
				//	There can be multiple portals between corner portals depending on the nearby obstacles.
				//
				//  The offsetting is controlled by how much the current portal segment (X) is visible from previous (P)
				//  and next (N) corner portals. The vertex can be freely moved on the visible portion of the portal
				//	without colliding the the corridor edges.
				
				const FReal PortalWidth = FVector::Distance(CurrPortal.Left, CurrPortal.Right);
				if (PortalWidth > UE_KINDA_SMALL_NUMBER)
				{
					FReal LeftVis = 0.0;
					FReal RightVis = 1.0;

					// Calculate how much the current portal is visible from previous and next corner portals.
					if (PrevCornerIndex != INDEX_NONE)
					{
						const FVector2D ForwardVis = UE::NavCorridor::Private::FindForwardVisibility(Portals, PrevCornerIndex, PortalIndex);
						LeftVis = FMath::Max(LeftVis, ForwardVis.X);
						RightVis = FMath::Min(RightVis, ForwardVis.Y);
					}

					const int32 NextCornerIndex = UE::NavCorridor::Private::FindNextCorner(Portals, PortalIndex);
					if (NextCornerIndex != INDEX_NONE)
					{
						const FVector2D BackwardVis = UE::NavCorridor::Private::FindBackwardVisibility(Portals, PortalIndex, NextCornerIndex);
						LeftVis = FMath::Max(LeftVis, BackwardVis.X);
						RightVis = FMath::Min(RightVis, BackwardVis.Y);
					}

					const FReal OffsetU = Offset / PortalWidth;

					// Find where the portals path location is along the portal. The offset direction depends whether the points is on left or right edge of the portal.
					FReal LocationU = UE::AI::ProjectPointOnSegment2D(CurrPortal.Location, CurrPortal.Left, CurrPortal.Right);
					if (LocationU < 0.5)
					{
						LocationU += OffsetU;
					}
					else
					{
						LocationU -= OffsetU;
					}
					LocationU = FMath::Clamp(LocationU, LeftVis, RightVis);

					CurrPortal.Location = FMath::Lerp(CurrPortal.Left, CurrPortal.Right, LocationU);
				}
			}

			// Move the in-between path locations on the line from previous corner portal to the newly offset location. 
			if (PrevCornerIndex != INDEX_NONE)
			{
				UE::NavCorridor::Private::InterpolateInBetweenLocations(Portals, PrevCornerIndex, PortalIndex);
			}
			
			PrevCornerIndex = PortalIndex;
		}
	}
}

FNavCorridorLocation FNavCorridor::FindNearestLocationOnPath(const FVector Location) const
{
	using FReal = FVector::FReal;

	if (Portals.Num() < 2)
	{
		return FNavCorridorLocation();
	}

	FReal NearestDistanceSq = MAX_dbl;
	FNavCorridorLocation Result;
	
	for (int32 PortalIndex = 0; PortalIndex < Portals.Num() - 1; PortalIndex++)
	{
		const FNavCorridorPortal& CurrPortal = Portals[PortalIndex];
		const FNavCorridorPortal& NextPortal = Portals[PortalIndex + 1];

		// Use the inverse bilinear to find nearest point on the sectors.
		// The nearest point is not euclidean nearest point, but nearest point terms of progression.
		// This ensures that there are no big jumps on inner corners, but transitions smooth.
		// @todo: Inv-bilinear is expensive, maybe something simpler as high level test. 
		const FVector2D UV = UE::AI::InvBilinear2DClamped(Location, CurrPortal.Left, CurrPortal.Right, NextPortal.Right, NextPortal.Left);
		FVector NearestSectionLocation = UE::AI::Bilinear(UV, CurrPortal.Left, CurrPortal.Right, NextPortal.Right, NextPortal.Left);
		const FReal SectionDistanceSq = FVector::DistSquared2D(Location, NearestSectionLocation);

		if (SectionDistanceSq < NearestDistanceSq)
		{
			NearestDistanceSq = SectionDistanceSq;
			Result.T = static_cast<float>(UV.Y);
			Result.Location = FMath::Lerp(CurrPortal.Location, NextPortal.Location, Result.T);
			Result.PortalIndex = PortalIndex;

			if (NearestDistanceSq < UE_KINDA_SMALL_NUMBER)
			{
				break;
			}
		}
	}

	return Result;
}

FNavCorridorLocation FNavCorridor::AdvancePathLocation(const FNavCorridorLocation& PathLocation, const FVector::FReal AdvanceDistance) const
{
	using FReal = FVector::FReal;

	if (Portals.Num() < 2 || !PathLocation.IsValid() || AdvanceDistance <= 0.0f)
	{
		return PathLocation;
	}
	
	FVector CurrentLocation = PathLocation.Location;
	FReal DistanceSoFar = 0.0;

	FNavCorridorLocation Result;
	
	for (int32 PortalIndex = PathLocation.PortalIndex + 1; PortalIndex < Portals.Num(); PortalIndex++)
	{
		const FNavCorridorPortal& CurrPortal = Portals[PortalIndex];
		const FReal SectionLength = FVector::Distance(CurrentLocation, CurrPortal.Location);
		if ((DistanceSoFar + SectionLength) > AdvanceDistance)
		{
			Result.T = FloatCastChecked<float>((AdvanceDistance - DistanceSoFar) / SectionLength, /* Precision */ 1./256.);
			Result.Location = FMath::Lerp(CurrentLocation, CurrPortal.Location, Result.T);
			Result.PortalIndex = PortalIndex - 1;
			break;
		}
		DistanceSoFar += SectionLength;
		CurrentLocation = CurrPortal.Location;
	}

	if (!Result.IsValid())
	{
		// Extrapolate last segment
		const FNavCorridorPortal& CurrPortal = Portals[Portals.Num() - 2];
		const FNavCorridorPortal& NextPortal = Portals[Portals.Num() - 1];
		const FReal SectionLength = FVector::Distance(NextPortal.Location, CurrPortal.Location);
		const FReal LeftoverDistance = AdvanceDistance - DistanceSoFar;
		Result.PortalIndex = Portals.Num() - 2;
		Result.T = static_cast<float>(1. + (LeftoverDistance / SectionLength)); // T will be > 1
		Result.Location = FMath::Lerp(CurrPortal.Location, NextPortal.Location, Result.T);
	}
	
	return Result;
}

double FNavCorridor::GetDistanceToEndOfPath(const FNavCorridorLocation& PathLocation) const
{
	using FReal = FVector::FReal;

	if (Portals.Num() < 2 || !PathLocation.IsValid())
	{
		return 0.0;
	}
	
	FVector CurrentLocation = PathLocation.Location;
	FReal DistanceSoFar = 0.0;

	for (int32 PortalIndex = PathLocation.PortalIndex + 1; PortalIndex < Portals.Num(); PortalIndex++)
	{
		const FNavCorridorPortal& CurrPortal = Portals[PortalIndex];
		const FReal SectionLength = FVector::Distance(CurrentLocation, CurrPortal.Location);
		DistanceSoFar += SectionLength;
		CurrentLocation = CurrPortal.Location;
	}

	return DistanceSoFar;
}

FVector FNavCorridor::GetPathDirection(const FNavCorridorLocation& PathLocation) const
{
	using FReal = FVector::FReal;

	if (Portals.Num() < 2 || !PathLocation.IsValid())
	{
		return FVector::ForwardVector;
	}

	return (Portals[PathLocation.PortalIndex + 1].Location - Portals[PathLocation.PortalIndex].Location).GetSafeNormal2D(); 
}

FVector FNavCorridor::ConstrainVisibility(const FNavCorridorLocation& PathLocation, const FVector Source, const FVector Target, const float ForceLookAheadDistance) const
{
	using FReal = FVector::FReal;
	
	if (Portals.Num() < 2 || !PathLocation.IsValid())
	{
		return Source;
	}

	const FReal SearchDistanceSq = ForceLookAheadDistance > UE_KINDA_SMALL_NUMBER ? FMath::Square(ForceLookAheadDistance) : FVector::DistSquared2D(Source, Target);

	// Find visibility cone up to Target
	bool bLeftDone = false;
	bool bRightDone = false;
	const FNavCorridorPortal& StartPortal = Portals[PathLocation.PortalIndex + 1];
	FVector ResultLeft = StartPortal.Left;
	FVector ResultRight = StartPortal.Right;
	FReal ResultLeftDistSq = FVector::DistSquared2D(Source, StartPortal.Left);
	FReal ResultRightDistSq = FVector::DistSquared2D(Source, StartPortal.Right);

	for (int32 PortalIndex = PathLocation.PortalIndex + 2; PortalIndex < Portals.Num(); PortalIndex++)
	{
		const FNavCorridorPortal& CurrPortal = Portals[PortalIndex];

		const FReal LeftDistSq = FVector::DistSquared2D(Source, CurrPortal.Left);
		const FReal RightDistSq = FVector::DistSquared2D(Source, CurrPortal.Right);
		
		bLeftDone |= LeftDistSq > SearchDistanceSq;
		bRightDone |= RightDistSq > SearchDistanceSq;

		if (!bLeftDone && UE::AI::TriArea2D(Source, ResultLeft, CurrPortal.Left) <= 0.0)
		{
			ResultLeft = CurrPortal.Left;
			ResultLeftDistSq = LeftDistSq;
		}

		if (!bRightDone && UE::AI::TriArea2D(Source, ResultRight, CurrPortal.Right) >= 0.0)
		{
			ResultRight = CurrPortal.Right;
			ResultRightDistSq = RightDistSq;
		}

		if (bLeftDone && bRightDone)
		{
			break;
		}
	}

	// If the vis cone collapsed, pick closest corner.
	if (UE::AI::TriArea2D(Source, ResultLeft, ResultRight) > 0.0)
	{
		if (ResultLeftDistSq < ResultRightDistSq)
		{
			ResultRight = ResultLeft; 
		}
		else
		{
			ResultLeft = ResultRight; 
		}
	}

	FVector Result = Target;

	if (ResultLeftDistSq < SearchDistanceSq && UE::AI::TriArea2D(Source, ResultLeft, Result) > 0.0f)
	{
		Result = ResultLeft;
	}

	if (ResultRightDistSq < SearchDistanceSq && UE::AI::TriArea2D(Source, ResultRight, Result) < 0.0f)
	{
		Result = ResultRight;
	}

	return Result;
}

bool FNavCorridor::HitTest(const FVector SegmentStart, const FVector SegmentEnd, double& HitT)
{
	using FReal = FVector::FReal;

	HitT = 1.0;
	bool bHit = false;
	
	for (int32 PortalIndex = 0; PortalIndex < Portals.Num() - 1; PortalIndex++)
	{
		const FNavCorridorPortal& CurrPortal = Portals[PortalIndex];
		const FNavCorridorPortal& NextPortal = Portals[PortalIndex + 1];

		FReal TA, TB;
		
		if (UE::AI::IntersectLineLine2D(SegmentStart, SegmentEnd, CurrPortal.Left, NextPortal.Left, TA, TB))
		{
			if (TB >= 0.0 && TB <= 1.0 && TA > 0.0 && TA < HitT)
			{
				HitT = TA;
				bHit = true;
			}
		}
		
		if (UE::AI::IntersectLineLine2D(SegmentStart, SegmentEnd, CurrPortal.Right, NextPortal.Right, TA, TB))
		{
			if (TB >= 0.0 && TB <= 1.0 && TA > 0.0 && TA < HitT)
			{
				HitT = TA;
				bHit = true;
			}
		}
	}

	return bHit;
}

