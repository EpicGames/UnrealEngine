// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavCorridorModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "NavCorridor"

void FNavCorridorModule::StartupModule()
{
}

void FNavCorridorModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FNavCorridorModule, NavCorridor)
