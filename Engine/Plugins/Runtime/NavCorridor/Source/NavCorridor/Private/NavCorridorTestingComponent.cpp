// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavCorridorTestingComponent.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "PrimitiveViewRelevance.h"
#include "NavMesh/NavMeshPath.h"
#include "NavigationSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavCorridorTestingComponent)

UNavCorridorTestingComponent::UNavCorridorTestingComponent(const FObjectInitializer& ObjectInitialize)
	: Super(ObjectInitialize)
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = true;
}

#if WITH_EDITOR
void UNavCorridorTestingComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (bUpdateParametersFromWidth)
	{
		CorridorParams.SetFromWidth(CorridorParams.Width);
	}
	
	UpdateTests();
}

void UNavCorridorTestingComponent::PostLoad()
{
	Super::PostLoad();
	UpdateTests();
}
#endif // WITH_EDITOR

void UNavCorridorTestingComponent::OnRegister()
{
	Super::OnRegister();
	UpdateTests();
}

void UNavCorridorTestingComponent::OnUnregister()
{
	Super::OnUnregister();
}

void UNavCorridorTestingComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	bool bShouldUpdate = false;
	
	if (bFindCorridorToGoal && !Corridor.IsValid())
	{
		bShouldUpdate = true;
	}

	if (bFollowPathOnGoalCorridor && !NearestPathLocation.IsValid())
	{
		bShouldUpdate = true;
	}

	if (GoalActor != nullptr && FVector::DistSquared(GoalActor->GetActorLocation(), LastTargetLocation) > FMath::Square(10.0f))
	{
		LastTargetLocation = GoalActor->GetActorLocation();
		bShouldUpdate = true;
	}

	if (bShouldUpdate)
	{
		UpdateTests();
	}
}


FBoxSphereBounds UNavCorridorTestingComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	const FVector ActorPosition = LocalToWorld.GetTranslation();
	FBox NewBounds(ActorPosition, ActorPosition);

	if (Path)
	{
		const TArray<FNavPathPoint>& PathPoints = Path->GetPathPoints();
		for (const FNavPathPoint& Point : PathPoints)
		{
			NewBounds += Point.Location;
		}
	}

	// Add some padding to make the whole path visible.
	NewBounds = NewBounds.ExpandBy(CorridorParams.Width * 0.5);
	
	return FBoxSphereBounds(NewBounds);
}

void UNavCorridorTestingComponent::UpdateNavData()
{
	// NavData already acquired.
	if (NavData != nullptr)
	{
		return;
	}

	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (NavSys == nullptr)
	{
		return;
	}
	
	const FVector ActorPosition = GetOwner()->GetActorLocation();
	NavData = NavSys->GetNavDataForProps(NavAgentProps, ActorPosition);
}

void UNavCorridorTestingComponent::UpdateTests()
{
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());

	UpdateNavData();

	Corridor.Reset();
	Path = nullptr;
	
	if (NavData == nullptr || NavSys == nullptr)
	{
		MarkRenderStateDirty();
		return;
	}

	const FVector ActorLocation = GetOwner()->GetActorLocation();
	const FVector GoalLocation = GoalActor ? GoalActor->GetActorLocation() : ActorLocation;

	if (bFindCorridorToGoal)
	{
		double StartTime = 0, EndTime = 0;
		
		StartTime = FPlatformTime::Seconds();

		const FPathFindingQuery PathQuery(this, *NavData, ActorLocation, GoalLocation, UNavigationQueryFilter::GetQueryFilter(*NavData, this, FilterClass));
		const FSharedConstNavQueryFilter NavQueryFilter = PathQuery.QueryFilter ? PathQuery.QueryFilter : NavData->GetDefaultQueryFilter();
		FPathFindingResult PathResult = NavSys->FindPathSync(NavAgentProps, PathQuery);

		Path = PathResult.Path;

		EndTime = FPlatformTime::Seconds();
		
		PathfindingTimeUs = FloatCastChecked<float>((EndTime - StartTime) * 1000000.0, UE::LWC::DefaultFloatPrecision); // us

		StartTime = FPlatformTime::Seconds();

		if (PathResult.IsSuccessful())
		{
			PathResult.Path->RemoveOverlappingPoints(FNavCorridor::OverlappingPointTolerance);
			Corridor.BuildFromPath(*Path, NavQueryFilter, CorridorParams);
			Corridor.OffsetPathLocationsFromWalls(PathOffset);
		}
		
		EndTime = FPlatformTime::Seconds();
		CorridorTimeUs = FloatCastChecked<float>((EndTime - StartTime) * 1000000.0, UE::LWC::DefaultFloatPrecision); // us
	}

	if (bFollowPathOnGoalCorridor)
	{
		const UNavCorridorTestingComponent* GoalComp = this;
		if (const UNavCorridorTestingComponent* Comp = GoalActor->FindComponentByClass<UNavCorridorTestingComponent>())
		{
			GoalComp = Comp;
		}
	
		NearestPathLocation.Reset();
		LookAheadPathLocation.Reset();

		if (GoalComp->Corridor.IsValid())
		{
			NearestPathLocation = GoalComp->Corridor.FindNearestLocationOnPath(ActorLocation);
			LookAheadPathLocation = GoalComp->Corridor.AdvancePathLocation(NearestPathLocation, FollowLookAheadDistance);
			ClampedLookAheadLocation = GoalComp->Corridor.ConstrainVisibility(NearestPathLocation, ActorLocation, LookAheadPathLocation.Location);
		}
	}

	MarkRenderStateDirty();
}


#if UE_ENABLE_DEBUG_DRAWING
FDebugRenderSceneProxy* UNavCorridorTestingComponent::CreateDebugSceneProxy()
{
	class FNavCorridorDebugRenderSceneProxy : public FDebugRenderSceneProxy
	{
	public:
		FNavCorridorDebugRenderSceneProxy(const UPrimitiveComponent* InComponent)
			: FDebugRenderSceneProxy(InComponent)
		{
		}
		
		virtual SIZE_T GetTypeHash() const override 
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override 
		{
			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = IsShown(View);
			Result.bDynamicRelevance = true;
			// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
			Result.bSeparateTranslucency = Result.bNormalTranslucency = IsShown(View) && GIsEditor;
			return Result;
		}
	};

	FNavCorridorDebugRenderSceneProxy* DebugProxy = new FNavCorridorDebugRenderSceneProxy(this);
	check(DebugProxy);

	const FVector ActorPosition = GetOwner()->GetActorLocation();

	DebugProxy->Spheres.Add(FDebugRenderSceneProxy::FSphere(50.0f, ActorPosition, FColor::Green));
	
	// Draw path
	if (Path)
	{
		const TArray<FNavPathPoint>& PathPoints = Path->GetPathPoints();
		for (int32 PointIndex = 0; PointIndex < PathPoints.Num(); PointIndex++)
		{
			const FNavPathPoint& PathPoint = PathPoints[PointIndex];

			if ((PointIndex + 1) < PathPoints.Num())
			{
				const FNavPathPoint& NextPathPoint = PathPoints[PointIndex + 1];
				DebugProxy->Lines.Add(FDebugRenderSceneProxy::FDebugLine(PathPoint.Location, NextPathPoint.Location, FColor(64,64,64), 2.f));
			}

			DebugProxy->Texts.Add(FDebugRenderSceneProxy::FText3d(FString::Printf(TEXT("%d-%d"), PointIndex, FNavMeshNodeFlags(Path->GetPathPoints()[PointIndex].Flags).AreaFlags), PathPoint.Location, FColor::White));
		}
	}

	// Draw corridor
	if (Corridor.IsValid())
	{
		const FVector Offset(0,0,10);

		for (int32 PortalIndex = 0; PortalIndex < Corridor.Portals.Num(); PortalIndex++)
		{
			const FNavCorridorPortal& Portal = Corridor.Portals[PortalIndex];

			// Portal segment
			DebugProxy->Lines.Add(FDebugRenderSceneProxy::FDebugLine(Offset + Portal.Left, Offset + Portal.Right, Portal.bIsPathCorner ? FColor::Red : FColor::Orange, Portal.bIsPathCorner ? 2.0f : 1.f));

			if ((PortalIndex + 1) < Corridor.Portals.Num())
			{
				// Sector boundaries
				const FNavCorridorPortal& NextPortal = Corridor.Portals[PortalIndex+1];
				DebugProxy->Lines.Add(FDebugRenderSceneProxy::FDebugLine(Offset + Portal.Left, Offset + NextPortal.Left, FColor::Orange, 2.f));
				DebugProxy->Lines.Add(FDebugRenderSceneProxy::FDebugLine(Offset + Portal.Right, Offset + NextPortal.Right, FColor::Orange, 2.f));

				// Path
				DebugProxy->Lines.Add(FDebugRenderSceneProxy::FDebugLine(Offset + Portal.Location, Offset + NextPortal.Location, FColor(255,128,128), 2.f));
			}
		}
	}

	if (NearestPathLocation.IsValid())
	{
		const FVector Offset(0,0,5);

		// Nearest path location
		DebugProxy->Lines.Add(FDebugRenderSceneProxy::FDebugLine( ActorPosition + Offset, NearestPathLocation.Location + Offset, FColor::Red, 1.0f));
		DebugProxy->Lines.Add(FDebugRenderSceneProxy::FDebugLine( NearestPathLocation.Location, NearestPathLocation.Location + Offset, FColor::Red, 2.0f));

		if (LookAheadPathLocation.IsValid())
		{
			const FVector Offset2(0,0,15);
			
			// Nearest path location to look ahead
			DebugProxy->Lines.Add(FDebugRenderSceneProxy::FDebugLine( NearestPathLocation.Location + Offset2, LookAheadPathLocation.Location + Offset2, FColor::Blue, 2.0f));
			DebugProxy->Lines.Add(FDebugRenderSceneProxy::FDebugLine( LookAheadPathLocation.Location, LookAheadPathLocation.Location + Offset2, FColor::Blue, 2.0f));

			// Clamped look ahead
			DebugProxy->Lines.Add(FDebugRenderSceneProxy::FDebugLine( ClampedLookAheadLocation, ClampedLookAheadLocation + Offset2, FColor::Magenta, 1.0f));
			DebugProxy->Lines.Add(FDebugRenderSceneProxy::FDebugLine( ActorPosition + Offset2, ClampedLookAheadLocation + Offset2, FColor::Magenta, 2.0f));
		}
	}

	
	return DebugProxy;
}
#endif // UE_ENABLE_DEBUG_DRAWING

ANavCorridorTestingActor::ANavCorridorTestingActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DebugComp = CreateDefaultSubobject<UNavCorridorTestingComponent>(TEXT("NavCorridorTestingComponent"));
	RootComponent = DebugComp;

	SetCanBeDamaged(false);
}

#if WITH_EDITOR
void ANavCorridorTestingActor::PostEditMove(bool bFinished)
{
	if (DebugComp)
	{
		DebugComp->UpdateTests();
	}
}
#endif

