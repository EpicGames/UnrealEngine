// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/Navigation/NavQueryFilter.h"
#include "NavCorridor.generated.h"

#define UE_API NAVCORRIDOR_API

struct FNavPathPoint;
struct FNavigationPath;


USTRUCT(BlueprintType)
struct FNavCorridorParams
{
	GENERATED_BODY()

	/** Sets good default values for the params based on corridor width. */
	void SetFromWidth(const float InWidth)
	{
		Width = InWidth;
		SmallSectorThreshold = Width * 0.3f;
		LargeSectorThreshold = Width;
		SimplifyEdgeThreshold = Width * 0.1f;
	}

	/** Points along the path which we want to visit instead of string pulling the location on the shortest path. */
	TArray<int32, TInlineAllocator<8>> PinnedPointsIndices;

	/** Width of the corridor to build */
	UPROPERTY(EditAnywhere, Category = "Pathfinding", meta=(ClampMin="10.0"))
	float Width = 200.0f;

	/** Distance to maintain between the path and the corridor boundaries. If the corridor is too tight, the offset will be reduced to the available space. */
	UPROPERTY(EditAnywhere, Category = "Pathfinding", meta = (ClampMin = "0.0"))
	float PathOffsetFromBoundaries = 0.f;

	/** How much the outer edges of obstacles are tapered out. This prevents small sectors and local traps.  */
	UPROPERTY(EditAnywhere, Category = "Pathfinding", meta=(ClampMin="10.0", ClampMax="45.0"))
	float ObstacleTaperAngle = 30.0f;

	/** Attempt to remove sectors narrower than this from the corridor. */
	UPROPERTY(EditAnywhere, Category = "Pathfinding", meta=(ClampMin="0.0"))
	float SmallSectorThreshold = 60.0f;

	/** Simplification is skipped if two neighbour sectors combined are longer than this. This ensures that long sectors do not lose volume due to simplification. */
	UPROPERTY(EditAnywhere, Category = "Pathfinding", meta=(ClampMin="0.0"))
	float LargeSectorThreshold = 200.0f;

	/** Corridor edge max simplification distance. */
	UPROPERTY(EditAnywhere, Category = "Pathfinding", meta=(ClampMin="0.0"))
	float SimplifyEdgeThreshold = 20.0f;

	/** If true do flip portals simplification. */
	UPROPERTY(EditAnywhere, Category = "Pathfinding")
	bool bSimplifyFlipPortals = true;
	
	/** If true do convex portals simplification. */
	UPROPERTY(EditAnywhere, Category = "Pathfinding")
	bool bSimplifyConvexPortals = true;
	
	/** If true do concave portals simplification. */
	UPROPERTY(EditAnywhere, Category = "Pathfinding")
	bool bSimplifyConcavePortals = true;

	/** Returns a string of the corridor parameters. */
	UE_API FString ToString() const;
};

/**
 * Portal of a section of the corridor. 
 */
struct FNavCorridorPortal
{
	/** Left side of the portal */
	FVector Left = FVector::ZeroVector;
	
	/** Righ side of the portal */
	FVector Right = FVector::ZeroVector;
	
	/** Path location at the portal */
	FVector Location = FVector::ZeroVector;
	
	/** Path point index of the original path */
	int16 PathPointIndex = 0;
	
	/** True if the portal is at original path point corner. */
	bool bIsPathCorner : 1 = false;

	/** When true, the string pulling will be forced to visit the original location at this portal, unless that location is outside the corridor (eg: too close to an obstacle). */
	bool bIsPinnedLocation : 1 = false;
};

/**
 * Location along the path through the corridor.
 */
struct FNavCorridorLocation
{
	bool IsValid() const { return PortalIndex != INDEX_NONE; }

	void Reset()
	{
		*this = FNavCorridorLocation();
	}

	/** Location on the path */
	FVector Location;

	/** Index of the start portal in the section where the location lies. */
	int32 PortalIndex = INDEX_NONE;

	/** Interpolation value representing the point between PortalIndex and PortalIndex+1 */
	float T = 0.0f;
};
	
/**
 * Navigation corridor defines free space around path. It is expanded from a string pulled path.
 * The corridor is represented as an array of portals, which leaves convex sectors in between them.
 */
struct FNavCorridor : public TSharedFromThis<FNavCorridor>
{
	/** @return true if the corridor is valid (has portals). */
	bool IsValid() const { return Portals.Num() > 1; }

	/** Resets and empties the corridor. */
	UE_API void Reset();

	/** Builds the corridor from a given Path. */
	UE_API void BuildFromPath(const FNavigationPath& Path, FSharedConstNavQueryFilter NavQueryFilter, const FNavCorridorParams& Params);

	/** Builds the corridor from a given array of Path points */
	UE_API void BuildFromPathPoints(const FNavigationPath& Path, TConstArrayView<FNavPathPoint> PathPoints, const int32 PathPointBaseIndex, FSharedConstNavQueryFilter NavQueryFilter, const FNavCorridorParams& Params);

	/** Offsets the path locations away from walls. */
	UE_API void OffsetPathLocationsFromWalls(const float Offset, bool bOffsetFirst = false, bool bOffsetLast = false);

	/** Finds nearest location on path going through the corridor. */
	UE_API FNavCorridorLocation FindNearestLocationOnPath(const FVector Location) const;

	/** Advances path location along the path that goes through the corridor. */
	UE_API FNavCorridorLocation AdvancePathLocation(const FNavCorridorLocation& PathLocation, const FVector::FReal AdvanceDistance) const;

	/** @return distance to the end of the path starting from the given the path location. */
	UE_API FVector::FReal GetDistanceToEndOfPath(const FNavCorridorLocation& PathLocation) const;

	/** @return direction of the path at given path location. */
	UE_API FVector GetPathDirection(const FNavCorridorLocation& PathLocation) const;

	/** @return target vector that is visible from Source. */
	UE_API FVector ConstrainVisibility(const FNavCorridorLocation& PathLocation, const FVector Source, const FVector Target, const float ForceLookAheadDistance = 0.0f) const;

	/** @return true if the segment hits the corridor boundary. */
	UE_API bool HitTest(const FVector SegmentStart, const FVector SegmentEnd, double& HitT);

	/** Portal points defining the corridor. */
	TArray<FNavCorridorPortal> Portals;

	/** Tolerance used to remove overlappint points in path before building the corridor (arbitrary value used to fix common cases).
	 * @todo: Make corridor support overlapping points and remove this.
	 */
	UE_API const static FVector OverlappingPointTolerance;
};

#undef UE_API
