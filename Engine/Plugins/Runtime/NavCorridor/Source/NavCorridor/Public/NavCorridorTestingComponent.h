// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/DebugDrawComponent.h"
#include "NavCorridor.h"
#include "AI/Navigation/NavigationTypes.h" // FNavAgentProperties
#include "NavCorridorTestingComponent.generated.h"

#define UE_API NAVCORRIDOR_API

class ANavigationData;
class FDebugRenderSceneProxy;

class UNavCorridorTestingComponent;
class ANavCorridorTestingActor;

/** Component for testing AI Locomotion functionality. */
UCLASS(MinimalAPI, ClassGroup = Custom, hidecategories = (Physics, Collision, Rendering, Cooking, Lighting, Navigation, Tags, HLOD, Mobile, AssetUserData, Activation))
class UNavCorridorTestingComponent : public UDebugDrawComponent
{
	GENERATED_BODY()
public:
	UE_API UNavCorridorTestingComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostLoad();
#endif // WITH_EDITOR

	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

#if UE_ENABLE_DEBUG_DRAWING
	UE_API FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
#endif // UE_ENABLE_DEBUG_DRAWING

	UE_API void UpdateTests();

protected:
	UE_API void UpdateNavData();
	
	UPROPERTY(EditAnywhere, Category = "Test")
	FNavAgentProperties NavAgentProps;

	UPROPERTY(EditAnywhere, Category = "Test")
	TSubclassOf<class UNavigationQueryFilter> FilterClass;

	/** If true, finds path to Goal actor. */
	UPROPERTY(EditAnywhere, Category = "Test");
	bool bFindCorridorToGoal = true;

	/** If true, finds nearest path location on Goal actor corridor. */
	UPROPERTY(EditAnywhere, Category = "Test");
	bool bFollowPathOnGoalCorridor = false;

	UPROPERTY(EditAnywhere, Category = "Test");
	float FollowLookAheadDistance = 200.0f;
	
	UPROPERTY(EditAnywhere, Category = "Test")
	TObjectPtr<AActor> GoalActor = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<ANavigationData> NavData;

	UPROPERTY(EditAnywhere, Category = "Test")
	FNavCorridorParams CorridorParams;

	UPROPERTY(EditAnywhere, Category = "Test");
	bool bUpdateParametersFromWidth = false;

	UPROPERTY(EditAnywhere, Category = "Test")
	float PathOffset = 40.0f;

	UPROPERTY(transient, VisibleAnywhere, BlueprintReadOnly, Category = "Profile")
	float PathfindingTimeUs = 0.0f;

	UPROPERTY(transient, VisibleAnywhere, BlueprintReadOnly, Category = "Profile")
	float CorridorTimeUs = 0.0f;
	
	FNavPathSharedPtr Path;
	FNavCorridor Corridor;

	/** Location used to track of the target actor moves. */
	FVector LastTargetLocation;

	FNavCorridorLocation NearestPathLocation;
	FNavCorridorLocation LookAheadPathLocation;
	FVector ClampedLookAheadLocation;
};

/** Debug actor to visually test zone graph. */
UCLASS(MinimalAPI, hidecategories = (Actor, Input, Collision, Rendering, Replication, Partition, HLOD, Cooking))
class ANavCorridorTestingActor : public AActor
{
	GENERATED_BODY()
public:
	UE_API ANavCorridorTestingActor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if WITH_EDITOR
	UE_API virtual void PostEditMove(bool bFinished) override;
#endif // WITH_EDITOR

protected:
	UPROPERTY(Category = Default, VisibleAnywhere, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UNavCorridorTestingComponent> DebugComp;
};

#undef UE_API
